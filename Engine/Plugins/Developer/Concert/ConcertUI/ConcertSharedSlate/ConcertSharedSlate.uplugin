{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.2",
	"FriendlyName": "Concert Shared Slate",
	"Description": "Contains UI that is shared for server and client UI modules",
	"Category": "Networking",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"Hidden": true,
	"IsBetaVersion": true,
	"Installed": false,
	"SupportedPrograms": [ "UnrealMultiUserServer", "UnrealMultiUserSlateServer", "UnrealRecoverySvc", "CrashReportClientEditor" ],
	"Modules": [
        {
          "Name": "ConcertSharedSlate",
          "Type": "UncookedOnly",
          "LoadingPhase": "Default",
          "ProgramAllowList": [ "UnrealMultiUserServer", "UnrealMultiUserSlateServer", "UnrealRecoverySvc", "CrashReportClientEditor" ]
        }
	],
	"Plugins": [
		{
			"Name": "ConcertMain",
			"Enabled": true
		},
		{
			"Name": "ConcertSyncCore",
			"Enabled": true
		}
	]
}
