// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ConcertSharedSlate : ModuleRules
	{
		public ConcertSharedSlate(ReadOnlyTargetRules Target) : base(Target)
		{
			ShortName = "ConShrSlate";

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Concert",
					"ConcertSyncCore",
					"Core",
					"CoreUObject",
					"Slate",
					"UndoHistory", 
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ConcertTransport",
					"ConcertSyncCore",
					"InputCore",
					"Json",
					"Projects",
					"SlateCore",
					"ToolMenus",
					"ToolWidgets"
				}
			);

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"Engine" // In editor builds, we want to display actors using their label. See DisplayUtils::GetObjectDisplayString
					}
				);
			}
		}
	}
}
