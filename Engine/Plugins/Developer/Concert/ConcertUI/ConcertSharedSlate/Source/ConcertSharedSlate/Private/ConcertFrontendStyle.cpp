// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertFrontendStyle.h"

#include "Brushes/SlateBoxBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"

LLM_DEFINE_TAG(Concert_ConcertFrontendStyle);

#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( FConcertFrontendStyle::InContent( RelativePath, ".png" ), __VA_ARGS__ )
#define IMAGE_BRUSH(RelativePath, ...) FSlateImageBrush(StyleSet->RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)
#define BOX_BRUSH(RelativePath, ... ) FSlateBoxBrush( StyleSet->RootToContentDir(RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define IMAGE_PLUGIN_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush( FConcertFrontendStyle::InContent(RelativePath, ".svg"), __VA_ARGS__)

FString FConcertFrontendStyle::InContent(const FString& RelativePath, const ANSICHAR* Extension)
{
	static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("ConcertSharedSlate"))->GetContentDir();
	return (ContentDir / RelativePath) + Extension;
}

TSharedPtr< class FSlateStyleSet > FConcertFrontendStyle::StyleSet;

FName FConcertFrontendStyle::GetStyleSetName()
{
	return FName(TEXT("ConcertFrontendStyle"));
}

void FConcertFrontendStyle::Initialize()
{
	LLM_SCOPE_BYTAG(Concert_ConcertFrontendStyle);
	// Only register once
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShared<FSlateStyleSet>(GetStyleSetName());
	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	// Const icon sizes
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon24x24(24.0f, 24.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon48x48(48.0f, 48.0f);

	// Use this to change the opacity. Ex: In UE4, the icon looked fine at 80%, in UE5, icons looks homogeneous with others at 100%.
	const FLinearColor IconColorAndOpacity(FLinearColor(1.f, 1.f, 1.f, 1.f));
	const FLinearColor ActiveIconColorAndOpacity(FLinearColor(1.f, 1.f, 1.f, 1.f));
	const FLinearColor ArchivedIconColorAndOpacity(FLinearColor(1.f, 1.f, 1.f, 1.f));

	// Colors
	StyleSet->Set("Concert.ActiveSession.Color", ActiveIconColorAndOpacity);
	StyleSet->Set("Concert.ArchivedSession.Color", ArchivedIconColorAndOpacity);

	// Expandable area
	StyleSet->Set("DetailsView.CollapsedCategory", new BOX_BRUSH( "Common/GroupBorder", FMargin(4.0f/16.0f) ) );
	StyleSet->Set("DetailsView.CategoryTop", new BOX_BRUSH( "PropertyView/DetailCategoryTop", FMargin( 4/16.0f, 8.0f/16.0f, 4/16.0f, 4/16.0f ) ) );
	StyleSet->Set("DetailsView.CollapsedCategory_Hovered", new BOX_BRUSH( "Common/GroupBorder", FMargin(4.0f/16.0f), FLinearColor(0.5f,0.5f,0.5f,1.0f)  ) );
	StyleSet->Set("DetailsView.CategoryTop_Hovered", new BOX_BRUSH( "PropertyView/DetailCategoryTop", FMargin( 4/16.0f, 8.0f/16.0f, 4/16.0f, 4/16.0f ), FLinearColor(0.5f,0.5f,0.5f,1.0f) ) );

	// 16x16
	StyleSet->Set("Concert.Persist",         new IMAGE_PLUGIN_BRUSH("Icons/icon_ConcertPersist_16x", Icon16x16));
	StyleSet->Set("Concert.LockBackground",  new IMAGE_PLUGIN_BRUSH("Icons/icon_ConcertLockBackground_16x", Icon16x16));
	StyleSet->Set("Concert.MyLock",          new IMAGE_PLUGIN_BRUSH("Icons/icon_ConcertMyLock_16x", Icon16x16));
	StyleSet->Set("Concert.OtherLock",       new IMAGE_PLUGIN_BRUSH("Icons/icon_ConcertOtherLock_16x", Icon16x16));
	StyleSet->Set("Concert.ModifiedByOther", new IMAGE_PLUGIN_BRUSH("Icons/icon_ConcertModifiedByOther_16x", Icon16x16));

	// Multi-user Tab/Menu icons
	StyleSet->Set("Concert.MultiUser", new IMAGE_PLUGIN_BRUSH_SVG("Icons/icon_MultiUser", Icon16x16));

	// Maps the UI Command name in Multi-User module. (UI_COMMAND does magic icon mapping when style name and command name matches)
	StyleSet->Set("Concert.OpenBrowser",  new IMAGE_PLUGIN_BRUSH("Icons/icon_MultiUser_32x", Icon16x16));
	StyleSet->Set("Concert.OpenSettings", new IMAGE_PLUGIN_BRUSH("Icons/icon_Settings_32x",  Icon16x16, IconColorAndOpacity));
	StyleSet->Set("Concert.LaunchServer", new IMAGE_PLUGIN_BRUSH("Icons/icon_NewServer_32x", Icon16x16, IconColorAndOpacity));

	// Multi-User Browser 
	StyleSet->Set("Concert.ArchiveSession",			new IMAGE_PLUGIN_BRUSH("Icons/icon_ArchiveSession_48x",			Icon24x24, IconColorAndOpacity));
	StyleSet->Set("Concert.CancelAutoJoin",			new IMAGE_PLUGIN_BRUSH("Icons/icon_CancelAutoJoin_48x",			Icon24x24, IconColorAndOpacity));
	StyleSet->Set("Concert.CloseServer",				new IMAGE_PLUGIN_BRUSH_SVG("Icons/icon_CloseServer_24",				Icon24x24, IconColorAndOpacity));
	StyleSet->Set("Concert.DeleteSession",			new IMAGE_PLUGIN_BRUSH("Icons/icon_DeleteSession_48x",			Icon24x24, IconColorAndOpacity));
	StyleSet->Set("Concert.JoinDefaultSession",		new IMAGE_PLUGIN_BRUSH("Icons/icon_JoinDefaultSession_48x",		Icon24x24, IconColorAndOpacity));
	StyleSet->Set("Concert.JoinSession",				new IMAGE_PLUGIN_BRUSH("Icons/icon_JoinSelectedSession_48x",	Icon24x24, IconColorAndOpacity));
	StyleSet->Set("Concert.LeaveSession",			new IMAGE_PLUGIN_BRUSH("Icons/icon_LeaveSession_48x",			Icon24x24, IconColorAndOpacity));
	StyleSet->Set("Concert.NewServer",				new IMAGE_PLUGIN_BRUSH("Icons/icon_NewServer_48x",				Icon24x24, IconColorAndOpacity));
	StyleSet->Set("Concert.NewSession",				new IMAGE_PLUGIN_BRUSH_SVG("Icons/icon_CreateMultiUser_24",		Icon24x24, IconColorAndOpacity));
	StyleSet->Set("Concert.PauseSession",			new IMAGE_PLUGIN_BRUSH("Icons/icon_PauseSession_48x",			Icon24x24, IconColorAndOpacity));
	StyleSet->Set("Concert.RestoreSession",			new IMAGE_PLUGIN_BRUSH("Icons/icon_RestoreSession_48x",			Icon24x24, IconColorAndOpacity));
	StyleSet->Set("Concert.ResumeSession",			new IMAGE_PLUGIN_BRUSH("Icons/icon_ResumeSession_48x",			Icon24x24, IconColorAndOpacity));
	StyleSet->Set("Concert.Settings",				new IMAGE_PLUGIN_BRUSH("Icons/icon_Settings_48x",				Icon24x24, IconColorAndOpacity));
	StyleSet->Set("Concert.NewServer.Small",			new IMAGE_PLUGIN_BRUSH("Icons/icon_NewServer_32x",				Icon16x16, IconColorAndOpacity));
	StyleSet->Set("Concert.NewSession.Small",		new IMAGE_PLUGIN_BRUSH_SVG("Icons/icon_CreateMultiUser_16",		Icon16x16, IconColorAndOpacity));
	StyleSet->Set("Concert.ActiveSession.Icon",		new IMAGE_PLUGIN_BRUSH_SVG("Icons/icon_SessionActive_16",		Icon16x16, ActiveIconColorAndOpacity));
	StyleSet->Set("Concert.ArchivedSession.Icon",	new IMAGE_PLUGIN_BRUSH_SVG("Icons/icon_SessionArchived_16",		Icon16x16, ArchivedIconColorAndOpacity));
	StyleSet->Set("Concert.SessionBrowser.FontSize", 11.f);
	StyleSet->Set("Concert.SessionRowPadding", FMargin(0.f, 4.f));
	
	// Multi-user Active session
	StyleSet->Set("Concert.JumpToLocation",     new IMAGE_PLUGIN_BRUSH("Icons/icon_PresenceLocation_32x",   Icon16x16, IconColorAndOpacity));
	StyleSet->Set("Concert.HidePresence",       new IMAGE_PLUGIN_BRUSH("Icons/icon_PresenceEyeOff_32x",     Icon16x16, IconColorAndOpacity));
	StyleSet->Set("Concert.ShowPresence",       new IMAGE_PLUGIN_BRUSH("Icons/icon_PresenceEyeOn_32x",      Icon16x16, IconColorAndOpacity));
	// Tab section of ActiveSession
	FCheckBoxStyle BaseCheckBoxStyle = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox");
	StyleSet->Set( "Concert.TabButton", BaseCheckBoxStyle
		.SetCheckedImage(FSlateRoundedBoxBrush(FStyleColors::Primary, 4.0f, FStyleColors::Primary, 1.0f))
		.SetCheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 4.0f, FStyleColors::PrimaryHover, 1.0f))
		.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 4.0f, FStyleColors::PrimaryHover, 1.0f))
		.SetPadding(FMargin(16, 6))
	);
	
	// Multi-user archived session browser
	StyleSet->Set("Concert.MuteActivities",     new IMAGE_PLUGIN_BRUSH("Icons/icon_PresenceEyeOff_32x",      Icon16x16, IconColorAndOpacity));
	StyleSet->Set("Concert.UnmuteActivities",     new IMAGE_PLUGIN_BRUSH("Icons/icon_PresenceEyeOn_32x",      Icon16x16, IconColorAndOpacity));

	// 24x24/48x48 -> For sequencer toolbar.
	StyleSet->Set("Concert.Sequencer.SyncTimeline",       new IMAGE_PLUGIN_BRUSH("Icons/icon_SequencerSyncTimeline_48x", Icon48x48, IconColorAndOpacity)); // Enable/disable playback and time scrubbing from a remote client.
	StyleSet->Set("Concert.Sequencer.SyncTimeline.Small", new IMAGE_PLUGIN_BRUSH("Icons/icon_SequencerSyncTimeline_48x", Icon24x24, IconColorAndOpacity));
	StyleSet->Set("Concert.Sequencer.SyncSequence",       new IMAGE_PLUGIN_BRUSH("Icons/icon_SequencerSyncSequence_48x", Icon48x48, IconColorAndOpacity)); // Allows or not a remote client to open/close sequencer.
	StyleSet->Set("Concert.Sequencer.SyncSequence.Small", new IMAGE_PLUGIN_BRUSH("Icons/icon_SequencerSyncSequence_48x", Icon24x24, IconColorAndOpacity));
	StyleSet->Set("Concert.Sequencer.SyncUnrelated",      new IMAGE_PLUGIN_BRUSH("Icons/icon_SequencerSyncUnrelated_48x", Icon48x48, IconColorAndOpacity)); // Enable/disable playback and time scrubbing from a remote client event if this user has a different sequence opened.
	StyleSet->Set("Concert.Sequencer.SyncUnrelated.Small",new IMAGE_PLUGIN_BRUSH("Icons/icon_SequencerSyncUnrelated_48x", Icon24x24, IconColorAndOpacity));

	// 40x40 -> Editor toolbar large icons.
	StyleSet->Set("Concert.Browse", new IMAGE_PLUGIN_BRUSH("Icons/icon_MultiUserMenuBrowse_40x", Icon40x40));
	StyleSet->Set("Concert.Join",   new IMAGE_PLUGIN_BRUSH("Icons/icon_MultiUserMenuJoin_40x",   Icon40x40));
	StyleSet->Set("Concert.Leave",  new IMAGE_PLUGIN_BRUSH("Icons/icon_MultiUserMenuLeave_40x",  Icon40x40));
	StyleSet->Set("Concert.Cancel", new IMAGE_PLUGIN_BRUSH("Icons/icon_MultiUserMenuCancel_40x", Icon40x40));

	// 20x20 -> Editor toolbar small icons.
	StyleSet->Set("Concert.Browse.Small", new IMAGE_PLUGIN_BRUSH("Icons/icon_MultiUserMenuBrowse_40x", Icon20x20));
	StyleSet->Set("Concert.Leave.Small",  new IMAGE_PLUGIN_BRUSH("Icons/icon_MultiUserMenuLeave_40x",  Icon20x20));
	StyleSet->Set("Concert.Join.Small",   new IMAGE_PLUGIN_BRUSH("Icons/icon_MultiUserMenuJoin_40x",   Icon20x20));
	StyleSet->Set("Concert.Cancel.Small", new IMAGE_PLUGIN_BRUSH("Icons/icon_MultiUserMenuCancel_40x", Icon20x20));

	// Disaster Recovery
	StyleSet->Set("Concert.RecoveryHub", new IMAGE_PLUGIN_BRUSH("Icons/icon_RecoveryHub_16x", Icon16x16));

	// Activity Text
	{
		FTextBlockStyle BoldText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");
		StyleSet->Set("ActivityText.Bold", FTextBlockStyle(BoldText).SetFont(FStyleFonts::Get().NormalBold));
	}

	// Colors
	{
		StyleSet->Set("Concert.Color.LocalUser", FLinearColor(0.31f, 0.749f, 0.333f));
		StyleSet->Set("Concert.Color.OtherUser", FLinearColor(0.93f, 0.608f, 0.169f));
	}

	// Colors
	StyleSet->Set("Concert.DisconnectedColor", FLinearColor(0.672f, 0.672f, 0.672f));

	// Default Concert Replication Columns Widths
	StyleSet->Set("Concert.Replication.Tree.RowHeight", 22.f);
	StyleSet->Set("Concert.Replication.Object.LabelSize", 1.5f);
	StyleSet->Set("Concert.Replication.Object.TypeWidth", 1.f);
	StyleSet->Set("Concert.Replication.Object.NumPropertiesSize", 1.f);
	StyleSet->Set("Concert.Replication.Property.LabelSize", 1.35f); // Property label needs to be smaller to align the top and bottom type columns
	StyleSet->Set("Concert.Replication.Property.TypeSize", 1.f);
	
	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
};

void FConcertFrontendStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}

TSharedPtr<class ISlateStyle> FConcertFrontendStyle::Get()
{
	return StyleSet;
}

#undef IMAGE_PLUGIN_BRUSH
#undef IMAGE_BRUSH

