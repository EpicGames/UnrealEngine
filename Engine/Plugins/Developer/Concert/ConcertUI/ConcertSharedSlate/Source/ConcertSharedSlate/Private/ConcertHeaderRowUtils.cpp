// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertHeaderRowUtils.h"

#include "Algo/AnyOf.h"
#include "Algo/ForEach.h"


#include "Framework/MultiBox/MultiBoxBuilder.h"

#include "Serialization/JsonSerializer.h"

#include "Widgets/Views/SHeaderRow.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConcertHeaderRowUtils)

#define LOCTEXT_NAMESPACE "ConcertHeaderRowUtils"

namespace UE::ConcertSharedSlate
{
	TSharedRef<SWidget> MakeTableContextMenu(const TSharedRef<SHeaderRow>& HeaderRow, TMap<FName, bool> ColumnsVisibleByDefault, bool bDefaultVisibility)
	{
		FMenuBuilder MenuBuilder(true, nullptr);
		AddDefaultControlEntries(HeaderRow, MenuBuilder, ColumnsVisibleByDefault, bDefaultVisibility);
		MenuBuilder.AddSeparator();
		AddEntriesForShowingHiddenRows(HeaderRow, MenuBuilder);
		return MenuBuilder.MakeWidget();
	}

	void AddDefaultControlEntries(const TSharedRef<SHeaderRow>& HeaderRow, FMenuBuilder& MenuBuilder, TMap<FName, bool> ColumnsVisibleByDefault, bool bDefaultVisibility)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("SelectAll", "Show all"),
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([WeakHeaderRow = TWeakPtr<SHeaderRow>(HeaderRow)]()
				{
					const TSharedPtr<SHeaderRow> HeaderRowPin = WeakHeaderRow.Pin();
					if (!ensure(HeaderRowPin))
					{
						return;
					}
					
					for (const SHeaderRow::FColumn& Column : HeaderRowPin->GetColumns())
					{
						HeaderRowPin->SetShowGeneratedColumn(Column.ColumnId, true);
					}
				}),
				FCanExecuteAction::CreateLambda([] { return true; })),
			NAME_None,
			EUserInterfaceActionType::Button
		);
		MenuBuilder.AddMenuEntry(
			LOCTEXT("HideAll", "Hide all"),
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([WeakHeaderRow = TWeakPtr<SHeaderRow>(HeaderRow)]()
				{
					const TSharedPtr<SHeaderRow> HeaderRowPin = WeakHeaderRow.Pin();
					if (!ensure(HeaderRowPin))
					{
						return;
					}
					
					for (const SHeaderRow::FColumn& Column : HeaderRowPin->GetColumns())
					{
						HeaderRowPin->SetShowGeneratedColumn(Column.ColumnId, false);
					}
				}),
				FCanExecuteAction::CreateLambda([] { return true; })),
			NAME_None,
			EUserInterfaceActionType::Button
		);
		MenuBuilder.AddMenuEntry(
			LOCTEXT("RestoreDefaultColumnVisibility", "Restore column visibility"),
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([WeakHeaderRow = TWeakPtr<SHeaderRow>(HeaderRow), ColumnsVisibleByDefault, bDefaultVisibility]
				{
					const TSharedPtr<SHeaderRow> HeaderRowPin = WeakHeaderRow.Pin();
					if (!ensure(HeaderRowPin))
					{
						return;
					}
					
					for (const SHeaderRow::FColumn& Column : HeaderRowPin->GetColumns())
					{
						const bool* IsVisible = ColumnsVisibleByDefault.Find(Column.ColumnId);
						HeaderRowPin->SetShowGeneratedColumn(Column.ColumnId, IsVisible ? *IsVisible : bDefaultVisibility);
					}
				}),
				FCanExecuteAction::CreateLambda([] { return true; })),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}

	void AddEntriesForShowingHiddenRows(const TSharedRef<SHeaderRow>& HeaderRow, FMenuBuilder& MenuBuilder)
	{
		for (int32 ColumnIndex = 0; ColumnIndex < HeaderRow->GetColumns().Num(); ++ColumnIndex)
		{
			const SHeaderRow::FColumn& Column = HeaderRow->GetColumns()[ColumnIndex];
			TWeakPtr<SHeaderRow> WeakHeaderRow = HeaderRow;
			if (!Column.ShouldGenerateWidget.IsSet())
			{
				MenuBuilder.AddMenuEntry(
					FText::Format(LOCTEXT("UnhideColumn", "Show \"{0}\" column"), Column.DefaultText.Get()),
					FText::Format(LOCTEXT("UnhideColumn_Tooltip", "Show the {0} column."), Column.DefaultText.Get()),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([WeakHeaderRow, ColumnID = Column.ColumnId, ColumnIndex](){ WeakHeaderRow.Pin()->SetShowGeneratedColumn(ColumnID, !WeakHeaderRow.Pin()->GetColumns()[ColumnIndex].bIsVisible); }),
						FCanExecuteAction::CreateLambda([] { return true; }),
						FIsActionChecked::CreateLambda([WeakHeaderRow, ColumnIndex] { return WeakHeaderRow.Pin()->GetColumns()[ColumnIndex].bIsVisible; })
						),
					NAME_None,
					EUserInterfaceActionType::ToggleButton
				);
			}
		}
	}

	static const FString JsonField_ColumnId("ColumnID");
	static const FString JsonField_IsVisible("bIsVisible");
	static const FString JsonField_Values("Values");

	FColumnVisibilitySnapshot SnapshotColumnVisibilityState(const TSharedRef<SHeaderRow>& HeaderRow)
	{
		TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();

		TArray<TSharedPtr<FJsonValue>> Array;
		for (const SHeaderRow::FColumn& Column : HeaderRow->GetColumns())
		{
			TSharedPtr<FJsonObject> ColumnObject = MakeShared<FJsonObject>();
			ColumnObject->SetStringField(JsonField_ColumnId, Column.ColumnId.ToString());
			ColumnObject->SetBoolField(JsonField_IsVisible, Column.bIsVisible);
			Array.Add(MakeShared<FJsonValueObject>(ColumnObject));
		}
		JsonObject->SetArrayField(JsonField_Values, Array);

		FString Snapshot;
		const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&Snapshot);
		FJsonSerializer::Serialize(JsonObject.ToSharedRef(), Writer);
		return { Snapshot };
	}

	void RestoreColumnVisibilityState(const TSharedRef<SHeaderRow>& HeaderRow, const FColumnVisibilitySnapshot& Snapshot)
	{
		TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
		const TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Snapshot.Snapshot);
		const bool bIsValid = FJsonSerializer::Deserialize(Reader, JsonObject);
		// This is a legal state, e.g. trying to restore a column view for the first time when no data is yet available
		if (!bIsValid)
		{
			return;
		}

		const TArray<TSharedPtr<FJsonValue>> ColumnObjects = JsonObject->GetArrayField(JsonField_Values);
		for (const TSharedPtr<FJsonValue>& ColumnValue : ColumnObjects)
		{
			TSharedPtr<FJsonObject>* PossibleColumnObject;
			if (!ColumnValue->TryGetObject(PossibleColumnObject))
			{
				continue;
			}
			const TSharedPtr<FJsonObject>& ColumnObject = *PossibleColumnObject;
			
			FString ColumnId;
			bool bIsVisible;
			if (!ColumnObject->TryGetStringField(JsonField_ColumnId, ColumnId) || !ColumnObject->TryGetBoolField(JsonField_IsVisible, bIsVisible))
			{
				continue;
			}

			HeaderRow->SetShowGeneratedColumn(*ColumnId, bIsVisible);
		}
	}

	FColumnVisibilityTransaction::FColumnVisibilityTransaction(TWeakPtr<SHeaderRow> HeaderRow)
		: HeaderRow(HeaderRow)
	{}

	FColumnVisibilityTransaction& FColumnVisibilityTransaction::SetHeaderRow(TWeakPtr<SHeaderRow> InHeaderRow)
	{
		HeaderRow = InHeaderRow;
		return *this;
	}

	FColumnVisibilityTransaction& FColumnVisibilityTransaction::SaveVisibilityAndSet(FName ColumnId, bool bShouldBeVisible)
	{
		if (const TSharedPtr<SHeaderRow> Pinned = HeaderRow.Pin())
		{
			SavedColumnVisibilities.Add(ColumnId, Pinned->IsColumnVisible(ColumnId));
			OverridenColumnVisibilities.Add(ColumnId, bShouldBeVisible);
			Pinned->SetShowGeneratedColumn(ColumnId, bShouldBeVisible);
		}
		return *this;
	}

	void FColumnVisibilityTransaction::ResetToSavedVisibilities(bool bOnlyResetIfNotOverriden)
	{
		const TSharedPtr<SHeaderRow> Pinned = HeaderRow.Pin();
		if (!Pinned)
		{
			return;
		}
		
		const bool bSkipUpdate = bOnlyResetIfNotOverriden
			&& Algo::AnyOf(OverridenColumnVisibilities, [this, &Pinned](const TPair<FName, bool>& SavedColumnEntry)
		{
			return Pinned->IsColumnVisible(SavedColumnEntry.Key) != SavedColumnEntry.Value;
		});
		if (!bSkipUpdate)
		{
			Algo::ForEach(SavedColumnVisibilities, [this, &Pinned](const TPair<FName, bool>& SavedColumnEntry)
			{
				Pinned->SetShowGeneratedColumn(SavedColumnEntry.Key, SavedColumnEntry.Value);
			});
		}
	}
}

#undef LOCTEXT_NAMESPACE
