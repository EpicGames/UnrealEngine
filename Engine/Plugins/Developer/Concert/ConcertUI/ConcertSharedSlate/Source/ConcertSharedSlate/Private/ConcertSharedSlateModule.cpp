// Copyright Epic Games, Inc. All Rights Reserved.

#include "IConcertSharedSlateModule.h"

class FConcertSharedSlateModule : public IConcertSharedSlateModule
{
public:
	FConcertSharedSlateModule() = default;
	virtual ~FConcertSharedSlateModule() {}

	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}
};


IMPLEMENT_MODULE(FConcertSharedSlateModule, ConcertSharedSlate);

