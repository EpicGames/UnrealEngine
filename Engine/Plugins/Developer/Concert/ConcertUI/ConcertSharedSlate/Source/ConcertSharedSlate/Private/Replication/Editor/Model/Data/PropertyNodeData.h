// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Editor/Model/Data/PropertyData.h"

#include "Containers/Array.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"

namespace UE::ConcertSharedSlate
{
	/** A node in the tree view which groups a subobject. */
	class FCategoryData
	{
	public:

		FCategoryData(TArray<TSoftObjectPtr<>> InContextObjects) : ContextObjects(MoveTemp(InContextObjects)) {}

		const TArray<TSoftObjectPtr<>>& GetContextObjects() const { return ContextObjects; }

	private:

		/** The objects this category represents; all the objects are releated, i.e. ActorA's StaticMeshComponent0 and ActorB's StaticMeshComponent0. */
		const TArray<TSoftObjectPtr<>> ContextObjects;
	};
	
	/** Instanced for each property row in IPropertyTreeView.*/
	class FPropertyNodeData
	{
	public:
		
		FPropertyNodeData(FPropertyData PropertyData)
			: PropertyData(MoveTemp(PropertyData))
		{}
		FPropertyNodeData(FCategoryData CategoryData)
			: CategoryData(MoveTemp(CategoryData))
		{}
		
		bool IsCategoryNode() const { return CategoryData.IsSet() && ensure(!PropertyData); }
		
		const TOptional<FPropertyData>& GetPropertyData() const { return PropertyData; }
		const TOptional<FCategoryData>& GetCategoryData() const { return CategoryData; }

	private:
		
		const TOptional<FPropertyData> PropertyData;
		const TOptional<FCategoryData> CategoryData;
	};
}