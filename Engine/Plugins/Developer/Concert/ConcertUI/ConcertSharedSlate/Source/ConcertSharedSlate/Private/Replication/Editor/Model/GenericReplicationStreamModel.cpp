// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericReplicationStreamModel.h"

#include "ConcertLogGlobal.h"
#include "SharedReplicationStreamModelGetters.h"
#include "Replication/PropertyChainUtils.h"
#include "Replication/Data/ObjectReplicationMap.h"
#include "Replication/Editor/Model/Extension/IStreamExtender.h"
#include "Replication/Editor/Model/Extension/IStreamExtensionContext.h"

#include "Algo/AllOf.h"
#include "Containers/Queue.h"

namespace UE::ConcertSharedSlate
{
	namespace Private
	{
		/** @return Whether any property was added */
		static bool AddParentProperties(const UStruct& Class, const FConcertPropertyChain& PropertyToAdd, TSet<FConcertPropertyChain>& ReplicatedProperties)
		{
			bool bAddedAtLeastOne = false;
			ConcertSyncCore::PropertyChain::ForEachReplicatableConcertProperty(Class, [&bAddedAtLeastOne, &ReplicatedProperties, &PropertyToAdd](FConcertPropertyChain&& Property)
				{
					if (Property.IsParentOf(PropertyToAdd))
					{
						bool bWasAlreadyInSet = false;
						ReplicatedProperties.Add(Property, &bWasAlreadyInSet);
						bAddedAtLeastOne |= !bWasAlreadyInSet;
					}
					return EBreakBehavior::Continue;
				});
			return bAddedAtLeastOne;
		}

		/** After removing RemovedProperty from Selection, walk up the chain property chain and remove any parent properties that now have 0 children. */
		static int32 RemoveParentPropertiesWithoutChildren(const FConcertPropertyChain& RemovedProperty, FConcertPropertySelection& Selection)
		{
			const auto HasNoChildren = [&Selection](const FConcertPropertyChain& CheckedProperty)
			{
				const bool bHasNoChildren = Algo::AllOf(Selection.ReplicatedProperties, [&CheckedProperty](const FConcertPropertyChain& SelectedProperty)
				{
					return !SelectedProperty.IsChildOf(CheckedProperty) || CheckedProperty == SelectedProperty;
				});
				return bHasNoChildren;
			};

			int32 NumRemoved = 0;
			bool bFollowChain = true;
			FConcertPropertyChain ParentProperty = RemovedProperty;
			while (!ParentProperty.IsEmpty() && bFollowChain)
			{
				ParentProperty = ParentProperty.GetParent();
				
				const bool bHasNoChildren = HasNoChildren(ParentProperty);
				if (bHasNoChildren)
				{
					++NumRemoved;
					Selection.ReplicatedProperties.Remove(ParentProperty);
				}

				// If current property still has children then transitively its parents will also continue to have a child
				bFollowChain = bHasNoChildren;
			}

			return NumRemoved;
		}
	}

	FGenericReplicationStreamModel::FGenericReplicationStreamModel(
		TAttribute<FConcertObjectReplicationMap*> InReplicationMapAttribute,
		TSharedPtr<IStreamExtender> InExtender
		)
		: ReplicationMapAttribute(MoveTemp(InReplicationMapAttribute))
		, Extender(MoveTemp(InExtender))
	{}

	FSoftClassPath FGenericReplicationStreamModel::GetObjectClass(const FSoftObjectPath& Object) const
	{
		return SharedStreamGetters::GetObjectClass(ReplicationMapAttribute.Get(), Object);
	}

	bool FGenericReplicationStreamModel::ContainsObjects(const TSet<FSoftObjectPath>& Objects) const
	{
		return SharedStreamGetters::ContainsObjects(ReplicationMapAttribute.Get(), Objects);
	}

	bool FGenericReplicationStreamModel::ContainsProperties(const FSoftObjectPath& Object, const TSet<FConcertPropertyChain>& Properties) const
	{
		return SharedStreamGetters::ContainsProperties(ReplicationMapAttribute.Get(), Object, Properties);
	}

	bool FGenericReplicationStreamModel::ForEachReplicatedObject(TFunctionRef<EBreakBehavior(const FSoftObjectPath& Object)> Delegate) const
	{
		return SharedStreamGetters::ForEachReplicatedObject(ReplicationMapAttribute.Get(), Delegate);
	}

	bool FGenericReplicationStreamModel::ForEachProperty(const FSoftObjectPath& Object, TFunctionRef<EBreakBehavior(const FConcertPropertyChain& Parent)> Delegate) const
	{
		return SharedStreamGetters::ForEachProperty(ReplicationMapAttribute.Get(), Object, Delegate);
	}

	uint32 FGenericReplicationStreamModel::GetNumProperties(const FSoftObjectPath& Object) const
	{
		return SharedStreamGetters::GetNumProperties(ReplicationMapAttribute.Get(), Object);
	}

	void FGenericReplicationStreamModel::AddObjects(TConstArrayView<UObject*> Objects)
	{
		FConcertObjectReplicationMap* ReplicationMap = ReplicationMapAttribute.Get();
		if (!ensure(ReplicationMap) || Objects.IsEmpty())
		{
			return;
		}
		
		TArray<UObject*> AddedObjects;
		for (UObject* Object : Objects)
		{
			const FSoftObjectPath ObjectPath = Object;
			if (ensureAlways(Object) && !ReplicationMap->ReplicatedObjects.Contains(ObjectPath))
			{
				FConcertReplicatedObjectInfo& ObjectInfo = ReplicationMap->ReplicatedObjects.Add(ObjectPath);
				ObjectInfo.ClassPath = Object->GetClass();
				AddedObjects.AddUnique(Object);
				
				ExtendObjects(*ReplicationMap, *Object, AddedObjects);
			}
		}
		
		if (!AddedObjects.IsEmpty())
		{
			OnObjectsChangedDelegate.Broadcast(AddedObjects, {}, EReplicatedObjectChangeReason::ChangedDirectly);
		}
	}

	void FGenericReplicationStreamModel::RemoveObjects(TConstArrayView<FSoftObjectPath> Objects)
	{
		FConcertObjectReplicationMap* ReplicationMap = ReplicationMapAttribute.Get();
		if (!ensure(ReplicationMap) || Objects.IsEmpty())
		{
			return;
		}
		
		TSet<FSoftObjectPath> ObjectsNotRemoved;
		for (const FSoftObjectPath& Object : Objects)
		{
			const bool bRemoved = ReplicationMap->ReplicatedObjects.Remove(Object) != 0;
			if (!bRemoved)
			{
				ObjectsNotRemoved.Add(Object);
			}
		}
		
		if (LIKELY(ObjectsNotRemoved.IsEmpty()))
		{
			OnObjectsChangedDelegate.Broadcast({}, Objects, EReplicatedObjectChangeReason::ChangedDirectly);
		}
		else if (ObjectsNotRemoved.Num() < Objects.Num())
		{
			// Uncommon case so it is ok if it is suboptimal
			TArray<FSoftObjectPath> Removed;
			for (const FSoftObjectPath& Object : Objects)
			{
				if (!ObjectsNotRemoved.Contains(Object))
				{
					Removed.Add(Object);
				}
			}
			OnObjectsChangedDelegate.Broadcast({}, Removed, EReplicatedObjectChangeReason::ChangedDirectly);
		}
	}

	void FGenericReplicationStreamModel::AddProperties(const FSoftObjectPath& Object, TConstArrayView<FConcertPropertyChain> Properties)
	{
		FConcertObjectReplicationMap* ReplicationMap = ReplicationMapAttribute.Get();
		if (!ensure(ReplicationMap))
		{
			return;
		}
		
		FConcertReplicatedObjectInfo* AssignedProperties = ReplicationMap->ReplicatedObjects.Find(Object);
		if (!AssignedProperties)
		{
			return;
		}

		UClass* Class = AssignedProperties->ClassPath.TryLoadClass<UObject>();
		if (!Class)
		{
			UE_LOGF(LogConcert, Warning, "FGenericReplicationStreamModel::AddProperties: Failed to resolve class %ls", *AssignedProperties->ClassPath.ToString());
			return;
		}

		bool bAddedAtLeastOne = false;
		TSet<FConcertPropertyChain>& ReplicatedProperties = AssignedProperties->PropertySelection.ReplicatedProperties;
		ReplicatedProperties.Reserve(ReplicatedProperties.Num() + Properties.Num());
		for (const FConcertPropertyChain& AddedProperty : Properties)
		{
			bool bWasAlreadyInSet = false;
			ReplicatedProperties.Add(AddedProperty, &bWasAlreadyInSet);
			
			bAddedAtLeastOne |= !bWasAlreadyInSet;
			// Parent properties must also be added
			// Not exactly efficient to iterate through the hierarchy for every removed item but it should be fine... Properties.Num() == 1 is the most common case
			bAddedAtLeastOne |= Private::AddParentProperties(*Class, AddedProperty, ReplicatedProperties);
		}

		if (bAddedAtLeastOne)
		{
			OnPropertiesChangedDelegate.Broadcast();
		}
	}

	void FGenericReplicationStreamModel::RemoveProperties(const FSoftObjectPath& Object, TConstArrayView<FConcertPropertyChain> Properties)
	{
		FConcertObjectReplicationMap* ReplicationMap = ReplicationMapAttribute.Get();
		if (!ensure(ReplicationMap))
		{
			return;
		}
		
		FConcertReplicatedObjectInfo* AssignedProperties = ReplicationMap->ReplicatedObjects.Find(Object);
		if (!AssignedProperties)
		{
			return;
		}

		bool bLoggedWarning = false;
		UClass* Class = AssignedProperties->ClassPath.TryLoadClass<UObject>();
		int32 NumRemoved = 0;
		for (const FConcertPropertyChain& RemovedProperty : Properties)
		{
			NumRemoved += AssignedProperties->PropertySelection.ReplicatedProperties.Remove(RemovedProperty);

			// Removal should not fail if the class is not available
			if (!Class)
			{
				UE_CLOGF(!bLoggedWarning, LogConcert, Warning, "FGenericReplicationStreamModel::RemoveProperties: Failed to resolve class %ls", *AssignedProperties->ClassPath.ToString());
				bLoggedWarning = true;
				continue;
			}

			// Child properties must also be removed
			// Not exactly efficient to iterate through the hierarchy for every removed item but it should be fine... Properties.Num() == 1 is the most common case
			ConcertSyncCore::PropertyChain::ForEachReplicatableConcertProperty(*Class, [&NumRemoved, &AssignedProperties, &RemovedProperty](FConcertPropertyChain&& Property)
			{
				if (Property.IsChildOf(RemovedProperty))
				{
					NumRemoved += AssignedProperties->PropertySelection.ReplicatedProperties.Remove(Property);
				}
				return EBreakBehavior::Continue;
			});

			NumRemoved += Private::RemoveParentPropertiesWithoutChildren(RemovedProperty, AssignedProperties->PropertySelection);
		}
		
		if (NumRemoved > 0)
		{
			OnPropertiesChangedDelegate.Broadcast();
		}
	}

	void FGenericReplicationStreamModel::ExtendObjects(FConcertObjectReplicationMap& ReplicationMap, UObject& AddedObject, TArray<UObject*>& ObjectsAddedSoFar)
	{
		if (!Extender)
		{
			return;
		}

		class FExtensionContext : public IStreamExtensionContext
		{
		public:

			TQueue<UObject*> ObjectsToProcess;
			FConcertObjectReplicationMap& ReplicationMap;
			TArray<UObject*>& ObjectsAddedSoFar;

			explicit FExtensionContext(FConcertObjectReplicationMap& ReplicationMap, TArray<UObject*>& ObjectsAddedSoFar)
				: ReplicationMap(ReplicationMap)
				, ObjectsAddedSoFar(ObjectsAddedSoFar)
			{}

			virtual void AddPropertyTo(UObject& Object, FConcertPropertyChain PropertyChain) override
			{
				UClass* ObjectClass = Object.GetClass();
				if (!ensure(ObjectClass))
				{
					return;
				}
				
				AddAdditionalObject(Object);
				FConcertReplicatedObjectInfo& ObjectInfo = ReplicationMap.ReplicatedObjects[&Object];

				constexpr bool bLog = false;
				const FProperty* ResolvedProperty = PropertyChain.ResolveProperty(*ObjectClass, bLog);
				if (!ResolvedProperty || !ConcertSyncCore::PropertyChain::IsReplicatableProperty(*ResolvedProperty))
				{
					UE_LOGF(LogConcert, Warning, "Property \"%ls\" is not a valid property to assign to object \"%ls\".", *PropertyChain.ToString(), *Object.GetPathName());
					return;
				}

				if (!ObjectInfo.PropertySelection.ReplicatedProperties.Contains(PropertyChain))
				{
					TSet<FConcertPropertyChain>& Properties = ObjectInfo.PropertySelection.ReplicatedProperties;
					Private::AddParentProperties(*ObjectClass, PropertyChain, Properties);
					Properties.Emplace(MoveTemp(PropertyChain));
				}
			}
			
			virtual void AddAdditionalObject(UObject& Object) override
			{
				if (!ObjectsAddedSoFar.Contains(&Object))
				{
					ObjectsAddedSoFar.AddUnique(&Object);
					ReplicationMap.ReplicatedObjects.FindOrAdd(&Object)
						.ClassPath = Object.GetClass();
					ObjectsToProcess.Enqueue(&Object);
				}
			}
		};

		FExtensionContext Context(ReplicationMap, ObjectsAddedSoFar);
		Context.ObjectsToProcess.Enqueue(&AddedObject);
		
		UObject* CurrentObject;
		while (Context.ObjectsToProcess.Dequeue(CurrentObject))
		{
			Extender->ExtendStream(*CurrentObject, Context);
		}
	}
}