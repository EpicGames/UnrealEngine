// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Editor/Model/IEditableReplicationStreamModel.h"

#include "Misc/Attribute.h"

struct FConcertStreamObjectAutoBindingRules;
struct FConcertObjectReplicationMap;

namespace UE::ConcertSharedSlate
{
	class IStreamExtender;

	/** Implements logic for editing a FConcertObjectReplicationMap. */
	class FGenericReplicationStreamModel
		: public IEditableReplicationStreamModel
	{
	public:
		
		FGenericReplicationStreamModel(
			TAttribute<FConcertObjectReplicationMap*> InReplicationMapAttribute,
			TSharedPtr<IStreamExtender> InExtender = nullptr
			);
		
		//~ Begin IReplicationStreamModel Interface
		virtual FSoftClassPath GetObjectClass(const FSoftObjectPath& Object) const override;
		virtual bool ContainsObjects(const TSet<FSoftObjectPath>& Objects) const override;
		virtual bool ContainsProperties(const FSoftObjectPath& Object, const TSet<FConcertPropertyChain>& Properties) const override;
		virtual bool ForEachReplicatedObject(TFunctionRef<EBreakBehavior(const FSoftObjectPath& Object)> Delegate) const override;
		virtual bool ForEachProperty(const FSoftObjectPath& Object, TFunctionRef<EBreakBehavior(const FConcertPropertyChain& Parent)> Delegate) const override;
		virtual uint32 GetNumProperties(const FSoftObjectPath& Object) const override;
		//~ End IReplicationStreamModel Interface
		
		//~ Begin IEditableReplicationStreamModel Interface
		virtual void AddObjects(TConstArrayView<UObject*> Objects) override;
		virtual void RemoveObjects(TConstArrayView<FSoftObjectPath> Objects) override;
		virtual void AddProperties(const FSoftObjectPath&, TConstArrayView<FConcertPropertyChain> Properties) override;
		virtual void RemoveProperties(const FSoftObjectPath&, TConstArrayView<FConcertPropertyChain> Properties) override;
		virtual FOnObjectsChanged& OnObjectsChanged() override { return OnObjectsChangedDelegate; }
		virtual FOnPropertiesChanged& OnPropertiesChanged() override { return OnPropertiesChangedDelegate; }
		//~ End IEditableReplicationStreamModel Interface

	private:

		/** Returns the replication map that is supposed to be edited. */
		const TAttribute<FConcertObjectReplicationMap*> ReplicationMapAttribute;

		/** Adds properties and objects when an object is added. Can be null. */
		TSharedPtr<IStreamExtender> Extender;

		FOnObjectsChanged OnObjectsChangedDelegate;
		FOnPropertiesChanged OnPropertiesChangedDelegate;

		/** Applies Extender to AddedObject whilst adding any additionally added UObjects to ObjectsAddedSoFar and avoiding adding objects already added to ObjectsAddedSoFar. */
		void ExtendObjects(FConcertObjectReplicationMap& ReplicationMap, UObject& AddedObject, TArray<UObject*>& ObjectsAddedSoFar);
	};
}