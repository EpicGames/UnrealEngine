// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Editor/Model/Object/IObjectHierarchyModel.h"

#include "UObject/SoftObjectPath.h"

namespace UE::ConcertSharedSlate
{
	void IObjectHierarchyModel::ForEachChildRecursive(
		const TSoftObjectPtr<>& Root,
		TFunctionRef<EBreakBehavior(const TSoftObjectPtr<>& Parent, const TSoftObjectPtr<>& ChildObject, EChildRelationship Relationship)> Callback,
		EChildRelationshipFlags InclusionFlags
		) const
	{
		struct FHelper
		{
			static EBreakBehavior Visit(
				const IObjectHierarchyModel& Model,
				const TSoftObjectPtr<>& Parent,
				const TSoftObjectPtr<>& ChildObject,
				EChildRelationship Relationship,
				TFunctionRef<EBreakBehavior(const TSoftObjectPtr<>&, const TSoftObjectPtr<>&, EChildRelationship)> Callback,
				EChildRelationshipFlags InclusionFlags
				)
			{
				if (Callback(Parent, ChildObject, Relationship) == EBreakBehavior::Break)
				{
					return EBreakBehavior::Break;
				}
				
				EBreakBehavior Result = EBreakBehavior::Continue;
				Model.ForEachDirectChild(ChildObject, [&Model, &ChildObject, &Callback, &InclusionFlags, &Result](const TSoftObjectPtr<>& ChildOfChild, EChildRelationship Relationship)
				{
					Result = Visit(Model, ChildObject, ChildOfChild, Relationship, Callback, InclusionFlags);
					return Result;
				}, InclusionFlags);
				return Result;
			}
		};
			
		ForEachDirectChild(Root, [this, &Root, &Callback, &InclusionFlags](const TSoftObjectPtr<>& ChildObject, EChildRelationship Relationship)
		{
			return FHelper::Visit(*this, Root, ChildObject, Relationship, Callback, InclusionFlags);
		}, InclusionFlags);
	}
}
