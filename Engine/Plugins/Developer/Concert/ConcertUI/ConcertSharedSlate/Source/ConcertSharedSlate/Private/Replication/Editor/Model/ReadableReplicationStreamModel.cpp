// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReadableReplicationStreamModel.h"

#include "SharedReplicationStreamModelGetters.h"

namespace UE::ConcertSharedSlate
{
	FReadableReplicationStreamModel::FReadableReplicationStreamModel(TAttribute<const FConcertObjectReplicationMap*> InReplicationMapAttribute)
		: ReplicationMapAttribute(MoveTemp(InReplicationMapAttribute))
	{
		check(ReplicationMapAttribute.IsBound());
	}

	FSoftClassPath FReadableReplicationStreamModel::GetObjectClass(const FSoftObjectPath& Object) const
	{
		return SharedStreamGetters::GetObjectClass(ReplicationMapAttribute.Get(), Object);
	}

	bool FReadableReplicationStreamModel::ContainsObjects(const TSet<FSoftObjectPath>& Objects) const
	{
		return SharedStreamGetters::ContainsObjects(ReplicationMapAttribute.Get(), Objects);
	}

	bool FReadableReplicationStreamModel::ContainsProperties(const FSoftObjectPath& Object, const TSet<FConcertPropertyChain>& Properties) const
	{
		return SharedStreamGetters::ContainsProperties(ReplicationMapAttribute.Get(), Object, Properties);
	}

	bool FReadableReplicationStreamModel::ForEachReplicatedObject(TFunctionRef<EBreakBehavior(const FSoftObjectPath& Object)> Delegate) const
	{
		return SharedStreamGetters::ForEachReplicatedObject(ReplicationMapAttribute.Get(), Delegate);
	}

	bool FReadableReplicationStreamModel::ForEachProperty(const FSoftObjectPath& Object, TFunctionRef<EBreakBehavior(const FConcertPropertyChain& Parent)> Delegate) const
	{
		return SharedStreamGetters::ForEachProperty(ReplicationMapAttribute.Get(), Object, Delegate);
	}

	uint32 FReadableReplicationStreamModel::GetNumProperties(const FSoftObjectPath& Object) const
	{
		return SharedStreamGetters::GetNumProperties(ReplicationMapAttribute.Get(), Object);
	}
}
