// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Editor/Model/IReplicationStreamModel.h"

#include "Misc/Attribute.h"

struct FConcertStreamObjectAutoBindingRules;
struct FConcertObjectReplicationMap;

namespace UE::ConcertSharedSlate
{
	class IStreamExtender;

	/** Implements logic for reading a FConcertObjectReplicationMap. */
	class FReadableReplicationStreamModel
		: public IReplicationStreamModel
	{
	public:
		
		FReadableReplicationStreamModel(TAttribute<const FConcertObjectReplicationMap*> InReplicationMapAttribute);
		
		//~ Begin IReplicationStreamModel Interface
		virtual FSoftClassPath GetObjectClass(const FSoftObjectPath& Object) const override;
		virtual bool ContainsObjects(const TSet<FSoftObjectPath>& Objects) const override;
		virtual bool ContainsProperties(const FSoftObjectPath& Object, const TSet<FConcertPropertyChain>& Properties) const override;
		virtual bool ForEachReplicatedObject(TFunctionRef<EBreakBehavior(const FSoftObjectPath& Object)> Delegate) const override;
		virtual bool ForEachProperty(const FSoftObjectPath& Object, TFunctionRef<EBreakBehavior(const FConcertPropertyChain& Parent)> Delegate) const override;
		virtual uint32 GetNumProperties(const FSoftObjectPath& Object) const override;
		//~ End IReplicationStreamModel Interface

	private:

		/** Returns the replication map that is supposed to be edited. */
		const TAttribute<const FConcertObjectReplicationMap*> ReplicationMapAttribute;
	};
}