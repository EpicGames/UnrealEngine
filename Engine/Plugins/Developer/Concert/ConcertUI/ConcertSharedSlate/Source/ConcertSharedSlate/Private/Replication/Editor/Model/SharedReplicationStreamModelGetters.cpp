// Copyright Epic Games, Inc. All Rights Reserved.

#include "SharedReplicationStreamModelGetters.h"

#include "Replication/Data/ObjectReplicationMap.h"

#include "Algo/AllOf.h"

namespace UE::ConcertSharedSlate::SharedStreamGetters
{
	FSoftClassPath GetObjectClass(const FConcertObjectReplicationMap* ReplicationMap, const FSoftObjectPath& Object)
	{
		if (!ensure(ReplicationMap))
		{
			return {};
		}
		
		const FConcertReplicatedObjectInfo* AssignedProperties = ReplicationMap->ReplicatedObjects.Find(Object);
		return AssignedProperties
			? AssignedProperties->ClassPath
			: FSoftClassPath{};
	}

	bool ContainsObjects(const FConcertObjectReplicationMap* ReplicationMap, const TSet<FSoftObjectPath>& Objects)
	{
		return ensure(ReplicationMap)
			&& Algo::AllOf(Objects, [ReplicationMap](const FSoftObjectPath& ObjectPath){ return ReplicationMap->ReplicatedObjects.Contains(ObjectPath); });
	}

	bool ContainsProperties(const FConcertObjectReplicationMap* ReplicationMap, const FSoftObjectPath& Object, const TSet<FConcertPropertyChain>& Properties)
	{
		if (!ensure(ReplicationMap))
		{
			return false;
		}

		const FConcertReplicatedObjectInfo* ObjectInfo = ReplicationMap->ReplicatedObjects.Find(Object);
		return ObjectInfo
			&& Algo::AllOf(Properties, [ObjectInfo](const FConcertPropertyChain& Property){ return ObjectInfo->PropertySelection.ReplicatedProperties.Contains(Property); });
	}

	bool ForEachReplicatedObject(const FConcertObjectReplicationMap* ReplicationMap, TFunctionRef<EBreakBehavior(const FSoftObjectPath& Object)> Delegate)
	{
		if (!ensure(ReplicationMap))
		{
			return false;
		}

		for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& ObjectMap: ReplicationMap->ReplicatedObjects)
		{
			if (Delegate(ObjectMap.Key) == EBreakBehavior::Break)
			{
				return true;
			}
		}

		return !ReplicationMap->ReplicatedObjects.IsEmpty();
	}

	bool ForEachProperty(const FConcertObjectReplicationMap* ReplicationMap, const FSoftObjectPath& Object, TFunctionRef<EBreakBehavior(const FConcertPropertyChain& Parent)> Delegate)
	{
		if (!ensure(ReplicationMap))
		{
			return false;
		}

		const FConcertReplicatedObjectInfo* AssignedProperties = ReplicationMap->ReplicatedObjects.Find(Object);
		if (!AssignedProperties)
		{
			return false;
		}

		for (const FConcertPropertyChain& ReplicatedPropertyInfo : AssignedProperties->PropertySelection.ReplicatedProperties)
		{
			if (Delegate(ReplicatedPropertyInfo) == EBreakBehavior::Break)
			{
				return true;
			}
		}
		return !AssignedProperties->PropertySelection.ReplicatedProperties.IsEmpty();
	}

	uint32 GetNumProperties(const FConcertObjectReplicationMap* ReplicationMap, const FSoftObjectPath& Object)
	{
		if (!ensure(ReplicationMap))
		{
			return 0;
		}
		
		const FConcertReplicatedObjectInfo* AssignedProperties = ReplicationMap->ReplicatedObjects.Find(Object);
		return AssignedProperties ? AssignedProperties->PropertySelection.ReplicatedProperties.Num() : 0;
	}
}
