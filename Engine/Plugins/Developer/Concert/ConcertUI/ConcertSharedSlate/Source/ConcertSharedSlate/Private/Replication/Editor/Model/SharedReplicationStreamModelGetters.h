// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"

#include "HAL/Platform.h"
#include "Templates/FunctionFwd.h"

struct FConcertPropertyChain;
struct FConcertObjectReplicationMap;
struct FSoftClassPath;
struct FSoftObjectPath;

enum class EBreakBehavior : uint8;

namespace UE::ConcertSharedSlate::SharedStreamGetters
{
	/** @return Gets the object class from ReplicationMap for Object. */
	FSoftClassPath GetObjectClass(const FConcertObjectReplicationMap* ReplicationMap, const FSoftObjectPath& Object);
	/** @return Whether all the objects are contained in ReplicationMap. */
	bool ContainsObjects(const FConcertObjectReplicationMap* ReplicationMap, const TSet<FSoftObjectPath>& Objects);
	/** @return Whether Object has the given properties assigned. */
	bool ContainsProperties(const FConcertObjectReplicationMap* ReplicationMap, const FSoftObjectPath& Object, const TSet<FConcertPropertyChain>& Properties);
	/** @return Whether Delegate was invokes at least once. */
	bool ForEachReplicatedObject(const FConcertObjectReplicationMap* ReplicationMap, TFunctionRef<EBreakBehavior(const FSoftObjectPath& Object)> Delegate);
	/** @return Whether Delegate was invokes at least once. */
	bool ForEachProperty(const FConcertObjectReplicationMap* ReplicationMap, const FSoftObjectPath& Object, TFunctionRef<EBreakBehavior(const FConcertPropertyChain& Parent)> Delegate);
	/** @return Number of properties assigned to Object */
	uint32 GetNumProperties(const FConcertObjectReplicationMap* ReplicationMap, const FSoftObjectPath& Object);
}
