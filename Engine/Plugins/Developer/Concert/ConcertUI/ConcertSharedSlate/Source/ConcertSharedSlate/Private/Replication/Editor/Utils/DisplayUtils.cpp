// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Editor/Utils/DisplayUtils.h"

#include "ConcertLogGlobal.h"
#include "Replication/PropertyResolutionCache.h"
#include "Misc/ObjectUtils.h"
#include "Replication/Editor/Model/IReplicationStreamModel.h"
#include "Replication/Editor/Model/Object/IObjectNameModel.h"

#include "Styling/SlateIconFinder.h"
#include "UObject/SoftObjectPtr.h"

#if WITH_EDITOR
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Blueprint/BlueprintSupport.h"
#include "GameFramework/Actor.h"
#endif

#if WITH_EDITORONLY_DATA
#include "UObject/UnrealType.h"
#endif

namespace UE::ConcertSharedSlate::DisplayUtils
{
	FText GetObjectDisplayText(const TSoftObjectPtr<>& Object, IObjectNameModel* Model)
	{
		if (Model)
		{
			return Model->GetObjectDisplayName(Object);
		}
		
		// Important! The object may not be loaded, yet. So only resolve, do not TryLoad. This could be if the asset is using a level that was not opened.
		if (const UObject* LoadedObject = Object.Get())
		{
			return FText::FromString(GetObjectDisplayString(*LoadedObject));
		}

		return ExtractObjectDisplayTextFromPath(Object.GetUniqueID());
	}
	
	FText ExtractObjectDisplayTextFromPath(const FSoftObjectPath& Object)
	{
		return FText::FromString(ConcertSyncCore::ExtractObjectNameFromPath(Object));
	}

	FString GetObjectDisplayString(const UObject& Object)
	{
		// In editor builds we have access to actor label so display them (that's also what's displayed in the scene outliner)
#if WITH_EDITOR 
		if (const AActor* AsActor = Cast<AActor>(&Object))
		{
			return AsActor->GetActorLabel();
		}
#endif
		// If we're not in the editor (or the object is not an actor) then whatever the name is in the soft object path will be good enough.
		return Object.GetName();
	}

	FText GetObjectTypeText(const FSoftClassPath& ClassPath)
	{
#if WITH_EDITOR
		if (ClassPath.IsValid())
		{
			// For C++ classes or loaded Blueprints
			if (UClass* Class = ClassPath.ResolveClass())
			{
				return Class->GetDisplayNameText();
			}

			// Assuming it is a Blueprint, try to get its display name from the registry tags
			const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
			const IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
			
			const FAssetData BlueprintData = AssetRegistry.GetAssetByObjectPath(ClassPath);
			if (BlueprintData.IsValid())
			{
				FString DisplayName;
				return BlueprintData.GetTagValue(FBlueprintTags::BlueprintDisplayName, DisplayName)
					? FText::FromString(DisplayName)
					: FText::GetEmpty();
			}

			// TODO: Check redirectors
			// It was not a Blueprint - the class was most likely removed
			UE_LOGF(LogConcert, Error, "Failed to get display name for unknown class \"%ls\"", *ClassPath.ToString());
		}
		
		return FText::FromName(NAME_None);
#else
		// On server the type info is not available. We can just display what it says in the path.
		return FText::FromString(ClassPath.GetAssetName());
#endif

	}

	FSlateIcon GetObjectIcon(const FSoftClassPath& ClassPath)
	{
		if (ClassPath.IsValid())
		{
			UClass* Class = ClassPath.ResolveClass();
			return Class
				? FSlateIconFinder::FindIconForClass(Class)
				: FSlateIcon();
		}
		return FSlateIcon();
	}

	FSlateIcon GetObjectIcon(UObject& Object)
	{
		return FSlateIconFinder::FindIconForClass(Object.GetClass());
	}
	
	FText GetPropertyDisplayText(ConcertSyncCore::PropertyChain::FPropertyResolutionCache& Cache, const FConcertPropertyChain& Property, UStruct* Class /*=nullptr*/)
	{
#if WITH_EDITORONLY_DATA
		// Class will (likely) be valid on editor builds but null on server
		const FProperty* ResolvedProperty = Class ? Cache.ResolveAndCache(*Class, Property) : nullptr;
		return ResolvedProperty
			? ResolvedProperty->GetDisplayNameText()
			:
#else
		return 
#endif
		FText::FromString(GetPropertyDisplayString(Cache, Property, Class));
    }
	
	FString GetPropertyDisplayString(ConcertSyncCore::PropertyChain::FPropertyResolutionCache& Cache, const FConcertPropertyChain& Property, UStruct* Class /*=nullptr*/)
	{
		SCOPED_CONCERT_TRACE(GetPropertyDisplayString);
		
#if WITH_EDITORONLY_DATA
		// Class will (likely) be valid on editor builds but null on server
		const FProperty* ResolvedProperty = Class ? Cache.ResolveAndCache(*Class, Property) : nullptr;
		return ResolvedProperty
			? ResolvedProperty->GetDisplayNameText().ToString()
			:
#else
		return 
#endif
		Property.ToString(FConcertPropertyChain::EToStringMethod::LeafProperty);
	}
}
