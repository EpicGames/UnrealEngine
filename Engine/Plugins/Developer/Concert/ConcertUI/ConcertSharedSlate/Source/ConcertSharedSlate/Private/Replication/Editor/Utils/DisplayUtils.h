// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/SoftObjectPtr.h"

class FString;
class UObject;
class UStruct;

struct FConcertPropertyChain;
struct FSlateIcon;
struct FSoftClassPath;
struct FSoftObjectPath;

namespace UE::ConcertSharedSlate
{
	class IObjectNameModel;
	class IReplicationStreamModel;
}
namespace UE::ConcertSyncCore::PropertyChain { class FPropertyResolutionCache; }

namespace UE::ConcertSharedSlate::DisplayUtils
{
	/** Unified version that reads from IObjectNameModel or defaults to extracting the name from the path. */
	FText GetObjectDisplayText(const TSoftObjectPtr<>& Object, IObjectNameModel* Model = nullptr);
	inline FText GetObjectDisplayText(const TSoftObjectPtr<>& Object, const TSharedPtr<IObjectNameModel>& Model = nullptr) { return GetObjectDisplayText(Object, Model.Get()); }
	
	/** @return The text to use for displaying this object's name */
	FText ExtractObjectDisplayTextFromPath(const FSoftObjectPath& Object);
	
	/** @return More lightweight version of GetObjectDisplayText which does not construct any FText. */
	FString GetObjectDisplayString(const UObject& Object);
	
	/** @return The text to use for displaying this object's type */
	FText GetObjectTypeText(const FSoftClassPath& ClassPath);
	
	/** @return The icon to use for this object */
	FSlateIcon GetObjectIcon(const FSoftClassPath& ClassPath);
	/** @return The icon to use for this object */
	FSlateIcon GetObjectIcon(UObject& Object);

	/** @return The text to use for displaying this property's name. Uses Class to determine class name if available. */
	FText GetPropertyDisplayText(ConcertSyncCore::PropertyChain::FPropertyResolutionCache& Cache, const FConcertPropertyChain& Property, UStruct* Class = nullptr);
	/** @return More lightweight version of GetPropertyDisplayString which does not construct any FText. Uses Class to determine class name if available. */
	FString GetPropertyDisplayString(ConcertSyncCore::PropertyChain::FPropertyResolutionCache& Cache, const FConcertPropertyChain& Property, UStruct* Class = nullptr);
}
