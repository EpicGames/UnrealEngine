// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyEnumerationUtils.h"

#include "Misc/EBreakBehavior.h"
#include "Replication/Editor/Model/IReplicationStreamModel.h"
#include "Replication/Editor/Model/Property/IPropertySource.h"
#include "Replication/Editor/Model/Property/IPropertySourceProcessor.h"

#include "Containers/Set.h"
#include "UObject/SoftObjectPath.h"

namespace UE::ConcertSharedSlate
{
	/** Gets the class from the model or loads it. This function is designed to be used without assuming that it is run in editor-builds. */
	FSoftClassPath GetObjectClassFromModelOrLoad(const TSoftObjectPtr<>& Object, const IReplicationStreamModel& Model)
	{
		const FSoftClassPath ResolvedClass = Model.GetObjectClass(Object.GetUniqueID());
		if (ResolvedClass.IsValid())
		{
			return ResolvedClass;
		}

#if WITH_EDITOR
		// The object is not yet in the model.
		const UObject* LoadedObject = Object.Get();
		return LoadedObject ? LoadedObject->GetClass() : FSoftClassPath{};
#else
		return FSoftClassPath{};
#endif
	}

	void EnumerateProperties(TConstArrayView<TSoftObjectPtr<>> Objects, const IReplicationStreamModel& Model, const IPropertySourceProcessor* OptionalSource, FEnumerateProperties Callback)
	{
		if (OptionalSource)
		{
			EnumerateAllProperties(Objects, *OptionalSource, Model, Callback);
		}
		else
		{
			EnumerateRegisteredPropertiesOnly(Objects, Model, Callback);
		}
	}

	/** Enumerates the properties that are assigned to the object in Model */
	void EnumerateRegisteredPropertiesOnly(TConstArrayView<TSoftObjectPtr<>> Objects, const IReplicationStreamModel& Model, FEnumerateProperties Callback)
	{
		for (const TSoftObjectPtr<>& Object : Objects)
		{
			const FSoftClassPath ObjectClass = GetObjectClassFromModelOrLoad(Object, Model);

			EBreakBehavior BreakBehavior = EBreakBehavior::Continue;
			Model.ForEachProperty(Object.GetUniqueID(), [&Callback, &ObjectClass, &BreakBehavior](const FConcertPropertyChain& Chain)
			{
				BreakBehavior = Callback(ObjectClass, Chain);
				return BreakBehavior;
			});

			if (BreakBehavior == EBreakBehavior::Break)
			{
				break;
			}
		}
	}

	/** Enumerate the properties that are selectable in Source (e.g. all properties in that class, @see FSelectPropertyFromUClassModel). */
	void EnumerateAllProperties(TConstArrayView<TSoftObjectPtr<>> Objects, const IPropertySourceProcessor& Source, const IReplicationStreamModel& Model, FEnumerateProperties Callback)
	{
		TSet<FSoftClassPath> VisitedClasses; 
			
		for (const TSoftObjectPtr<>& Object : Objects)
		{
			const FSoftClassPath ObjectClass = GetObjectClassFromModelOrLoad(Object, Model);
			const FPropertySourceContext ObjectQueryContext(Object, ObjectClass);
					
			EBreakBehavior BreakBehavior = EBreakBehavior::Continue;
			Source.ProcessPropertySource(ObjectQueryContext, [&Callback, &ObjectClass, &BreakBehavior](const IPropertySource& PropertySource)
			{
				PropertySource.EnumerateProperties([&Callback, &ObjectClass, &BreakBehavior](const FPropertyInfo& PropertyInfo)
				{
					BreakBehavior = Callback(ObjectClass, PropertyInfo.Property);
					return BreakBehavior;
				});
			});
					
			if (BreakBehavior == EBreakBehavior::Break)
			{
				break;
			}
		}
	}
}