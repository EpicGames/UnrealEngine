// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "HAL/Platform.h"
#include "Templates/FunctionFwd.h"
#include "UObject/SoftObjectPtr.h"

enum class EBreakBehavior : uint8;
struct FConcertPropertyChain;
struct FSoftClassPath;
struct FSoftObjectPath;

namespace UE::ConcertSharedSlate
{
	class IPropertySourceProcessor;
	class IReplicationStreamModel;
	using FEnumerateProperties = TFunctionRef<EBreakBehavior(const FSoftClassPath&, const FConcertPropertyChain&)>;
	
	/** Gets the class from the model or loads it. This function is designed to be used without assuming that it is run in editor-builds. */
	FSoftClassPath GetObjectClassFromModelOrLoad(const TSoftObjectPtr<>& Object, const IReplicationStreamModel& Model);

	/** Calls EnumerateRegisteredPropertiesOnly or EnumerateAllProperties depending on whether OptionalSource is nullptr. */
	void EnumerateProperties(TConstArrayView<TSoftObjectPtr<>> Objects, const IReplicationStreamModel& Model, const IPropertySourceProcessor* OptionalSource, FEnumerateProperties Callback);
	/** Enumerates the properties that are assigned to the object in Model */
	void EnumerateRegisteredPropertiesOnly(TConstArrayView<TSoftObjectPtr<>> Objects, const IReplicationStreamModel& Model, FEnumerateProperties Callback);
	/** Enumerate the properties that are selectable in Source (e.g. all properties in that class, @see FSelectPropertyFromUClassModel). */
	void EnumerateAllProperties(TConstArrayView<TSoftObjectPtr<>> Objects, const IPropertySourceProcessor& Source, const IReplicationStreamModel& Model, FEnumerateProperties Callback);
}