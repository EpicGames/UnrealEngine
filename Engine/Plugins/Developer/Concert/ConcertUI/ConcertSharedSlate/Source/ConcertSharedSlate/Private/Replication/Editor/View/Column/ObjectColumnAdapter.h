// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Editor/Model/Data/ReplicatedObjectData.h"
#include "Replication/Editor/View/Column/IObjectTreeColumn.h"
#include "Replication/Editor/View/Column/IReplicationTreeColumn.h"

namespace UE::ConcertSharedSlate
{
	/** Adapts an IObjectTreeColumn to IReplicationTreeColumn<FPropertyData>. It simply passes additional info down to IPropertyTreeColumn. */
	class FObjectColumnAdapter : public IReplicationTreeColumn<FReplicatedObjectData>
	{
	public:

		static TArray<TReplicationColumnEntry<FReplicatedObjectData>> Transform(const TArray<FObjectColumnEntry>& Entries)
		{
			TArray<TReplicationColumnEntry<FReplicatedObjectData>> Result;
			Algo::Transform(Entries, Result, [](const FObjectColumnEntry& Entry) -> TReplicationColumnEntry<FReplicatedObjectData>
			{
				return {
					TReplicationColumnDelegates<FReplicatedObjectData>::FCreateColumn::CreateLambda([CreateDelegate = Entry.CreateColumn]()
						{
							return MakeShared<FObjectColumnAdapter>(CreateDelegate.Execute());
						}),
					Entry.ColumnId,
					Entry.ColumnInfo
				};
			});
			return Result;
		}
		
		FObjectColumnAdapter(TSharedRef<IObjectTreeColumn> InAdaptedColumn)
			: AdaptedColumn(MoveTemp(InAdaptedColumn))
		{}
		
		virtual SHeaderRow::FColumn::FArguments CreateHeaderRowArgs() const override { return AdaptedColumn->CreateHeaderRowArgs(); }
		virtual TSharedRef<SWidget> GenerateColumnWidget(const FBuildArgs& InArgs) override
		{
			return AdaptedColumn->GenerateColumnWidget({ InArgs.HighlightText, Transform(InArgs.RowItem) });
		}
		virtual void PopulateSearchString(const FReplicatedObjectData& InItem, TArray<FString>& InOutSearchStrings) const override
		{
			return AdaptedColumn->PopulateSearchString(Transform(InItem), InOutSearchStrings);
		}

		virtual bool CanBeSorted() const override { return AdaptedColumn->CanBeSorted(); }
		virtual bool IsLessThan(const FReplicatedObjectData& Left, const FReplicatedObjectData& Right) const override { return AdaptedColumn->IsLessThan(Transform(Left), Transform(Right)); }

	private:

		TSharedRef<IObjectTreeColumn> AdaptedColumn;

		static FObjectTreeRowContext Transform(FReplicatedObjectData Data)
		{
			return { MoveTemp(Data) };
		}
	};
}
