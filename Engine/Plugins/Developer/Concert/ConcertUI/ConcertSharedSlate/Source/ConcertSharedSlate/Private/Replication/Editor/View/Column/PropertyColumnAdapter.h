// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Editor/Model/Data/PropertyData.h"
#include "Replication/Editor/Model/Data/PropertyNodeData.h"
#include "Replication/Editor/View/Column/IPropertyTreeColumn.h"
#include "Replication/Editor/View/Column/IReplicationTreeColumn.h"

namespace UE::ConcertSharedSlate
{
	/** Adapts an IPropertyTreeColumn to IReplicationTreeColumn<FPropertyData>. It simply passes additional info down to IPropertyTreeColumn. */
	class FPropertyColumnAdapter : public IReplicationTreeColumn<FPropertyNodeData>
	{
	public:

		static TArray<TReplicationColumnEntry<FPropertyNodeData>> Transform(const TArray<FPropertyColumnEntry>& Entries)
		{
			TArray<TReplicationColumnEntry<FPropertyNodeData>> Result;
			Algo::Transform(Entries, Result, [](const FPropertyColumnEntry& Entry) -> TReplicationColumnEntry<FPropertyNodeData>
			{
				return {
					TReplicationColumnDelegates<FPropertyNodeData>::FCreateColumn::CreateLambda([CreateDelegate = Entry.CreateColumn]()
					{
						return MakeShared<FPropertyColumnAdapter>(CreateDelegate.Execute());
					}),
					Entry.ColumnId,
					Entry.ColumnInfo
				};
			});
			return Result;
		}
		
		FPropertyColumnAdapter(TSharedRef<IPropertyTreeColumn> InAdaptedColumn)
			: AdaptedColumn(MoveTemp(InAdaptedColumn))
		{}
		
		virtual SHeaderRow::FColumn::FArguments CreateHeaderRowArgs() const override { return AdaptedColumn->CreateHeaderRowArgs(); }
		virtual TSharedRef<SWidget> GenerateColumnWidget(const FBuildArgs& InArgs) override
		{
			return AdaptedColumn->GenerateColumnWidget({ InArgs.HighlightText, Transform(InArgs.RowItem) });
		}
		virtual void PopulateSearchString(const FPropertyNodeData& InItem, TArray<FString>& InOutSearchStrings) const override
		{
			return AdaptedColumn->PopulateSearchString(Transform(InItem), InOutSearchStrings);
		}

		virtual bool CanBeSorted() const override { return AdaptedColumn->CanBeSorted(); }
		virtual bool IsLessThan(const FPropertyNodeData& Left, const FPropertyNodeData& Right) const override { return AdaptedColumn->IsLessThan(Transform(Left), Transform(Right)); }

	private:

		TSharedRef<IPropertyTreeColumn> AdaptedColumn;

		static FPropertyTreeRowContext Transform(const FPropertyNodeData& Data)
		{
			const TOptional<FPropertyData>& PropertyData = Data.GetPropertyData();
			checkf(PropertyData, TEXT("This node is not a property node - column callbacks should not have been invoked on it!"));
			return { *PropertyData };
		}
	};
}
