// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Editor/View/Extension/CategoryRowGeneration.h"

#include "Replication/Editor/Model/Object/IObjectNameModel.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::ConcertSharedSlate
{
	namespace Private
	{
		class SDefaultCategoryRow : public SCompoundWidget, public ICategoryRow
		{
			FText Label;
		public:

			SLATE_BEGIN_ARGS(SDefaultCategoryRow){}
				SLATE_ARGUMENT(FText, Label)
			SLATE_END_ARGS()

			void Construct(const FArguments& InArgs)
			{
				Label = InArgs._Label;
				ChildSlot
				[
					// Do not give this widget any highlight text because category rows are not part of the search
					SNew(STextBlock)
					.Text(Label)
					.Font(FAppStyle::Get().GetFontStyle("DetailsView.CategoryFontStyle"))
					.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
				];
			}

			//~ Begin ICategoryRow Interface
			virtual void GenerateSearchTerms(const TArray<TSoftObjectPtr<>>& ContextObjects, TArray<FString>& InOutSearchTerms) const override
			{
				const FString LabelString = Label.ToString();
				InOutSearchTerms.Add(LabelString);
				
				TArray<FString> Split;
				LabelString.ParseIntoArray(Split, TEXT(" "));
				InOutSearchTerms.Append(Split);
			}
			virtual TSharedRef<SWidget> GetWidget() override { return SharedThis(this); }
			//~ End ICategoryRow Interface
		};
	}
	
	FCreateCategoryRow CreateDefaultCategoryGenerator(TSharedRef<IObjectNameModel> NameModel)
	{
		return FCreateCategoryRow::CreateLambda([NameModel](const FCategoryRowGenerationArgs& Args) -> TSharedRef<ICategoryRow>
		{
			const TArray<TSoftObjectPtr<>>& ContextObjects = Args.ContextObjects;
			const FText Label = ContextObjects.IsEmpty()
				? FText::GetEmpty()
				: NameModel->GetObjectDisplayName(ContextObjects[0]);

			// Do not give this widget any highlight text because category rows are not part of the search
			return SNew(Private::SDefaultCategoryRow)
				.Label(Label);
		});
	}
}

