// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Editor/View/IReplicationStreamViewer.h"

namespace UE::ConcertSharedSlate
{
	TArray<FSoftObjectPath> IReplicationStreamViewer::GetObjectsBeingPropertyEdited() const
	{
		TArray<FSoftObjectPath> Result;
		Algo::Transform(GetSelectedObjects(), Result, [](const TSoftObjectPtr<>& Object){ return Object.GetUniqueID(); });
		return Result;
	}
}
