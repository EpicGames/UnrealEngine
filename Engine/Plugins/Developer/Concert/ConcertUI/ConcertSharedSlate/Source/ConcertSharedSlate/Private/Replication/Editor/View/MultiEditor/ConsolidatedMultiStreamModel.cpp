// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConsolidatedMultiStreamModel.h"

#include "Replication/Editor/Model/IEditableMultiReplicationStreamModel.h"
#include "Replication/Editor/Model/IEditableReplicationStreamModel.h"

#include "UObject/Package.h"

namespace UE::ConcertSharedSlate
{
	FConsolidatedMultiStreamModel::FConsolidatedMultiStreamModel(
		TSharedRef<IEditableReplicationStreamModel> InConsolidatedObjectModel, 
		TSharedRef<IEditableMultiReplicationStreamModel> InMultiStreamModel,
		FGetAutoAssignTarget InGetAutoAssignTargetDelegate
		)
		: ConsolidatedStreamModel(MoveTemp(InConsolidatedObjectModel))
		, MultiStreamModel(MoveTemp(InMultiStreamModel))
		, GetAutoAssignTargetDelegate(MoveTemp(InGetAutoAssignTargetDelegate))
	{
		MultiStreamModel->OnStreamExternallyChanged().AddRaw(this, &FConsolidatedMultiStreamModel::OnStreamExternallyChanged);
		MultiStreamModel->OnStreamSetChanged().AddRaw(this, &FConsolidatedMultiStreamModel::RebuildStreamSubscriptions);
		ConsolidatedStreamModel->OnObjectsChanged().AddRaw(this, &FConsolidatedMultiStreamModel::OnObjectsChanged_StreamForAdding);
		RebuildStreamSubscriptions();
	}

	FConsolidatedMultiStreamModel::~FConsolidatedMultiStreamModel()
	{
		MultiStreamModel->OnStreamExternallyChanged().RemoveAll(this);
		MultiStreamModel->OnStreamSetChanged().RemoveAll(this);
		ConsolidatedStreamModel->OnObjectsChanged().RemoveAll(this);
		ClearStreamSubscriptions();
	}

	FSoftClassPath FConsolidatedMultiStreamModel::GetObjectClass(const FSoftObjectPath& Object) const
	{
		FSoftClassPath Result = ConsolidatedStreamModel->GetObjectClass(Object);
		if (Result.IsValid())
		{
			return Result;
		}

		MultiStreamModel->ForEachStream([&Object, &Result](const TSharedRef<IReplicationStreamModel>& Stream)
        {
        	Result = Stream->GetObjectClass(Object);
        	return Result.IsValid() ? EBreakBehavior::Break : EBreakBehavior::Continue;
        });
		return Result;
	}

	bool FConsolidatedMultiStreamModel::ContainsObjects(const TSet<FSoftObjectPath>& Objects) const
	{
		if (ConsolidatedStreamModel->ContainsObjects(Objects))
		{
			return true;
		}

		bool bContains = false;
		MultiStreamModel->ForEachStream([&Objects, &bContains](const TSharedRef<IReplicationStreamModel>& Stream)
		{
			bContains = Stream->ContainsObjects(Objects);
			return bContains ? EBreakBehavior::Break : EBreakBehavior::Continue;
		});
		return bContains;
	}

	bool FConsolidatedMultiStreamModel::ContainsProperties(const FSoftObjectPath& Object, const TSet<FConcertPropertyChain>& Properties) const
	{
		bool bContainsProperty = false;
		MultiStreamModel->ForEachStream([&Object, &Properties, &bContainsProperty](const TSharedRef<IReplicationStreamModel>& Model)
		{
			bContainsProperty = Model->ContainsProperties(Object, Properties);
			return bContainsProperty ? EBreakBehavior::Break : EBreakBehavior::Continue;
		});
		return bContainsProperty;
	}

	bool FConsolidatedMultiStreamModel::ForEachReplicatedObject(TFunctionRef<EBreakBehavior(const FSoftObjectPath& Object)> Delegate) const
	{
		TSet<FSoftObjectPath> UniquePathsOnly;
		EBreakBehavior BreakBehavior = EBreakBehavior::Continue;
		const auto ProcessObjects = [&Delegate, &UniquePathsOnly, &BreakBehavior](const FSoftObjectPath& Object)
		{
			bool bAlreadyInSet = false;
			UniquePathsOnly.Add(Object, &bAlreadyInSet);
			if (!bAlreadyInSet)
			{
				BreakBehavior = Delegate(Object);
			}
			
			return BreakBehavior;
		};
		
		bool bAnyMappings = ConsolidatedStreamModel->ForEachReplicatedObject(ProcessObjects);
		if (BreakBehavior == EBreakBehavior::Break)
		{
			return bAnyMappings;
		}

		MultiStreamModel->ForEachStream([&BreakBehavior, &ProcessObjects, &bAnyMappings](const TSharedRef<IReplicationStreamModel>& Stream)
		{
			bAnyMappings |= Stream->ForEachReplicatedObject(ProcessObjects);
			return BreakBehavior;
		});
		
		return bAnyMappings;
	}

	bool FConsolidatedMultiStreamModel::ForEachProperty(const FSoftObjectPath& Object, TFunctionRef<EBreakBehavior(const FConcertPropertyChain& Property)> Delegate) const
	{
		bool bCalledAtLeastOnce = false;
		MultiStreamModel->ForEachStream([&Object, &Delegate, &bCalledAtLeastOnce](const TSharedRef<IReplicationStreamModel>& Model)
		{
			EBreakBehavior BreakBehavior = EBreakBehavior::Continue;
			bCalledAtLeastOnce |= Model->ForEachProperty(Object, [&Delegate, &BreakBehavior](const FConcertPropertyChain& Property)
			{
				BreakBehavior = Delegate(Property);
				return BreakBehavior;
			});
			return BreakBehavior;
		});
		return bCalledAtLeastOnce;
	}

	uint32 FConsolidatedMultiStreamModel::GetNumProperties(const FSoftObjectPath& Object) const
	{
		uint32 PropertyUpperLimit = 0;
		MultiStreamModel->ForEachStream([&Object, &PropertyUpperLimit](const TSharedRef<IReplicationStreamModel>& Model)
		{
			PropertyUpperLimit += Model->GetNumProperties(Object);
			return EBreakBehavior::Continue;
		});

		// This is needed, so we don't count properties double (two separate streams may contain the same property twice).
		TSet<FConcertPropertyChain> Properties;
		Properties.Reserve(PropertyUpperLimit);
		MultiStreamModel->ForEachStream([&Object, &Properties](const TSharedRef<IReplicationStreamModel>& Model)
		{
			Model->ForEachProperty(Object, [&Properties](const FConcertPropertyChain& Property)
			{
				Properties.Add(Property);
				return EBreakBehavior::Continue;
			});
			return EBreakBehavior::Continue;
		});
		
		return Properties.Num();
	}

	void FConsolidatedMultiStreamModel::AddObjects(TConstArrayView<UObject*> Objects)
	{
		const TSharedPtr<IEditableReplicationStreamModel> TargetStream = GetAutoAssignTargetDelegate.IsBound()
			? GetAutoAssignTargetDelegate.Execute(Objects)
			: nullptr;
		if (TargetStream && ensure(MultiStreamModel->GetEditableStreams().Contains(TargetStream.ToSharedRef())))
		{
			TargetStream->AddObjects(Objects);
		}
		else
		{
			ConsolidatedStreamModel->AddObjects(Objects);
		}
	}

	void FConsolidatedMultiStreamModel::RemoveObjects(TConstArrayView<FSoftObjectPath> Objects)
	{
		ConsolidatedStreamModel->RemoveObjects(Objects);
		for (const TSharedRef<IEditableReplicationStreamModel>& Model : MultiStreamModel->GetEditableStreams())
		{
			Model->RemoveObjects(Objects);
		}
	}

	void FConsolidatedMultiStreamModel::OnObjectsChanged_StreamForAdding(TConstArrayView<UObject*> AddedObjects, TConstArrayView<FSoftObjectPath> RemovedObjects, EReplicatedObjectChangeReason ChangeReason)
	{
		OnObjectsChangedDelegate.Broadcast(AddedObjects, RemovedObjects, ChangeReason);
	}

	void FConsolidatedMultiStreamModel::OnObjectsChanged_ClientStream(TConstArrayView<UObject*> AddedObjects, TConstArrayView<FSoftObjectPath> RemovedObjects, EReplicatedObjectChangeReason ChangeReason)
	{
		OnObjectsChangedDelegate.Broadcast(AddedObjects, RemovedObjects, ChangeReason);
	}

	void FConsolidatedMultiStreamModel::OnStreamExternallyChanged(TSharedRef<IReplicationStreamModel> Stream)
	{
		OnObjectsChangedDelegate.Broadcast({}, {}, EReplicatedObjectChangeReason::ExternalChange);
	}

	void FConsolidatedMultiStreamModel::RebuildStreamSubscriptions()
	{
		ClearStreamSubscriptions();

		for (const TSharedRef<IEditableReplicationStreamModel>& Model : MultiStreamModel->GetEditableStreams())
		{
			Model->OnObjectsChanged().AddRaw(this, &FConsolidatedMultiStreamModel::OnObjectsChanged_ClientStream);
		}
	}

	void FConsolidatedMultiStreamModel::ClearStreamSubscriptions()
	{
		for (const TWeakPtr<IEditableReplicationStreamModel>& Model : SubscribedToViewers)
		{
			if (const TSharedPtr<IEditableReplicationStreamModel> ModelPin = Model.Pin())
			{
				ModelPin->OnObjectsChanged().RemoveAll(this);
			}
		}
		SubscribedToViewers.Empty();
	}
}
