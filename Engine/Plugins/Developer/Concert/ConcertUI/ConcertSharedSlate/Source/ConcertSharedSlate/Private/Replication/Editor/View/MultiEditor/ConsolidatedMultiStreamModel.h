// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/ReplicationWidgetFactories.h"
#include "Replication/Editor/Model/IEditableReplicationStreamModel.h"

class UReplicationStreamObject;
namespace UE::ConcertSharedSlate { class IEditableMultiReplicationStreamModel; }

namespace UE::ConcertSharedSlate
{
	/**
	 * Combines multiple streams into one and allows you to query whether an object is contained by any of them
	 * Does not NOT support property queries.
	 *
	 * Implements functions in a specific way to allow editing of multiple streams in a single SBaseReplicationStreamEditor UI.
	 * Coupled to SMultiReplicationStreamEditor.
	 *
	 * How this works:
	 * - AddObject & RemoveObject calls are forwarded to ConsolidatedStreamModel
	 * - Object getters, like ForEachReplicatedObject, query ConsolidatedStreamModel AND the multiple streams in MultiStreamModel
	 * - Property editing is not supported
	 * 
	 * SMultiReplicationStreamEditor's responsibility is to consolidate all replicated objects in a single UI and display all properties in the class.
	 * The creator of SMultiReplicationStreamEditor is supposed to inject custom columns into the property section for assigning specific properties
	 * to the streams in the IEditableReplicationStreamModel. 
	 */
	class FConsolidatedMultiStreamModel
		: public IEditableReplicationStreamModel
	{
	public:

		FConsolidatedMultiStreamModel(
			TSharedRef<IEditableReplicationStreamModel> InConsolidatedModel,
			TSharedRef<IEditableMultiReplicationStreamModel> InMultiStreamModel,
			FGetAutoAssignTarget InGetAutoAssignTargetDelegate
			);
		virtual ~FConsolidatedMultiStreamModel() override;
		
		//~ Begin IReplicationStreamModel Interface
		virtual FSoftClassPath GetObjectClass(const FSoftObjectPath& Object) const override;
		virtual bool ContainsObjects(const TSet<FSoftObjectPath>& Objects) const override;
		virtual bool ContainsProperties(const FSoftObjectPath& Object, const TSet<FConcertPropertyChain>& Properties) const override;
		virtual bool ForEachReplicatedObject(TFunctionRef<EBreakBehavior(const FSoftObjectPath& Object)> Delegate) const override;
		virtual bool ForEachProperty(const FSoftObjectPath& Object, TFunctionRef<EBreakBehavior(const FConcertPropertyChain& Property)> Delegate) const override;
		virtual uint32 GetNumProperties(const FSoftObjectPath& Object) const override;
		//~ End IReplicationStreamModel Interface

		//~ Begin IEditableReplicationStreamModel Interface
		virtual void AddObjects(TConstArrayView<UObject*> Objects) override;
		virtual void RemoveObjects(TConstArrayView<FSoftObjectPath> Objects) override;
		virtual void AddProperties(const FSoftObjectPath& Object, TConstArrayView<FConcertPropertyChain> Properties) override { ensure(false); }
		virtual void RemoveProperties(const FSoftObjectPath& Object, TConstArrayView<FConcertPropertyChain> Properties) override { ensure(false); }
		virtual FOnObjectsChanged& OnObjectsChanged() override { return OnObjectsChangedDelegate; }
		virtual FOnPropertiesChanged& OnPropertiesChanged() override { return OnPropertiesChangedDelegate; }
		//~ End IEditableReplicationStreamModel Interface

	private:

		/**
		 * AddObjects calls are redirected to this model.
		 * That allows objects added via the add button to show up in the UI regardless of what the real streams contain.
		 *
		 * RemoveObjects are applied to both StreamForAdding and to the editable streams in MultiStreamModel.
		 */
		TSharedRef<IEditableReplicationStreamModel> ConsolidatedStreamModel;

		/** Contains all the streams that are being displayed */
		TSharedRef<IEditableMultiReplicationStreamModel> MultiStreamModel;

		/** Called the equivalent event of ConsolidatedStreamModel or any of the streams in MultiStreamModel is broadcast. */
		FOnObjectsChanged OnObjectsChangedDelegate;
		/** Never actually called, just to implement OnPropertiesChanged. */
		FOnPropertiesChanged OnPropertiesChangedDelegate;
		/** If set, also adds newly added objects to the target stream. */
		FGetAutoAssignTarget GetAutoAssignTargetDelegate;
		
		/** All streams we've subscribed to for changes. Used so we can unsubscribe when MultiStreamModel->OnStreamsChanged triggers. */
		TArray<TWeakPtr<IEditableReplicationStreamModel>> SubscribedToViewers;

		// Trigger our delegates when any of the substreams change.
		void OnObjectsChanged_StreamForAdding(TConstArrayView<UObject*> AddedObjects, TConstArrayView<FSoftObjectPath> RemovedObjects, EReplicatedObjectChangeReason ChangeReason);
		void OnObjectsChanged_ClientStream(TConstArrayView<UObject*> AddedObjects, TConstArrayView<FSoftObjectPath> RemovedObjects, EReplicatedObjectChangeReason ChangeReason);
		
		/** Trigger subscription rebuild when streams are added / removed. */
		void OnStreamExternallyChanged(TSharedRef<IReplicationStreamModel> Stream);
		void RebuildStreamSubscriptions();
		void ClearStreamSubscriptions();
	};
}

