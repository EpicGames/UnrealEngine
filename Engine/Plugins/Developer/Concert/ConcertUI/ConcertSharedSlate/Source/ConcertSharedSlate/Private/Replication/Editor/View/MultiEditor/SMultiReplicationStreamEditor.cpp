// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMultiReplicationStreamEditor.h"

#include "ConsolidatedMultiStreamModel.h"
#include "Replication/ReplicationWidgetFactories.h"
#include "Replication/Editor/Model/IEditableMultiReplicationStreamModel.h"
#include "Replication/Editor/View/IReplicationStreamEditor.h"

namespace UE::ConcertSharedSlate
{
	void SMultiReplicationStreamEditor::Construct(const FArguments& InArgs, FCreateMultiStreamEditorParams EditorParams, FCreateViewerParams ViewerParams)
	{
		MultiStreamModel = MoveTemp(EditorParams.MultiStreamModel);
		ConsolidatedModel = MakeShared<FConsolidatedMultiStreamModel>(MoveTemp(EditorParams.ConsolidatedObjectModel), MoveTemp(EditorParams.MultiStreamModel), EditorParams.GetAutoAssignToStreamDelegate);
		
		FCreateEditorParams BaseEditorParams
		{
			.DataModel = ConsolidatedModel.ToSharedRef(),
			.ObjectSource = MoveTemp(EditorParams.ObjectSource),
			.PropertySource = MoveTemp(EditorParams.PropertySource),
			.OnPreAddSelectedObjectsDelegate = MoveTemp(EditorParams.OnPreAddSelectedObjectsDelegate),
			.OnPostAddSelectedObjectsDelegate = MoveTemp(EditorParams.OnPostAddSelectedObjectsDelegate)
		};
		EditorView = CreateBaseStreamEditor(MoveTemp(BaseEditorParams), MoveTemp(ViewerParams));

		ChildSlot
		[
			EditorView.ToSharedRef()
		];
	}
}
