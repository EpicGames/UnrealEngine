// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConsolidatedMultiStreamModel.h"
#include "Replication/ReplicationWidgetFactories.h"
#include "Replication/Editor/View/IMultiReplicationStreamEditor.h"
#include "Replication/Editor/View/IReplicationStreamEditor.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::ConcertSharedSlate
{
	enum class EReplicatedObjectChangeReason : uint8;
	class FConsolidatedMultiStreamModel;
	class IEditableReplicationStreamModel;
}

namespace UE::ConcertSharedSlate
{
	struct FCreateMultiStreamEditorParams;

	/**
	 * Displays multiple streams in a single widget.
	 * 
	 * SMultiReplicationStreamEditor's responsibility is to consolidate all replicated objects in a single UI and display all properties in the class.
	 * The creator of SMultiReplicationStreamEditor is supposed to inject custom columns into the property section for assigning specific properties
	 * to the streams in the IEditableReplicationStreamModel. 
	 */
	class SMultiReplicationStreamEditor : public IMultiReplicationStreamEditor
	{
		using Super = IMultiReplicationStreamEditor;
	public:

		SLATE_BEGIN_ARGS(SMultiReplicationStreamEditor)
		{}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, FCreateMultiStreamEditorParams EditorParams, FCreateViewerParams ViewerParams);

		virtual IReplicationStreamEditor& GetEditorBase() const override { return *EditorView; }
		virtual IEditableMultiReplicationStreamModel& GetMultiStreamModel() const override { return *MultiStreamModel; }
		virtual IEditableReplicationStreamModel& GetConsolidatedModel() const override { return *ConsolidatedModel; }

	private:

		/** This model combines all of the user provided ones so they are all displayed in one view. */
		TSharedPtr<FConsolidatedMultiStreamModel> ConsolidatedModel;
		
		/** Tells us which streams there are. */
		TSharedPtr<IEditableMultiReplicationStreamModel> MultiStreamModel;

		/** The main view, which is displaying ConsolidatedModel. */
		TSharedPtr<IReplicationStreamEditor> EditorView;
	};
}
