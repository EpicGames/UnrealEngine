// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBaseReplicationStreamEditor.h"

#include "Model/Item/SourceModelBuilders.h"
#include "Replication/Editor/Model/Data/ReplicatedObjectData.h"
#include "Replication/Editor/Utils/DisplayUtils.h"
#include "Replication/Editor/Model/IEditableReplicationStreamModel.h"
#include "Replication/Editor/Model/Object/IObjectHierarchyModel.h"
#include "Replication/Editor/Model/Property/IPropertySourceProcessor.h"
#include "Replication/Editor/Model/ObjectSource/IObjectSelectionSourceModel.h"
#include "Replication/Editor/View/ObjectViewer/SReplicationStreamViewer.h"

#include "Algo/AnyOf.h"
#include "Misc/ObjectUtils.h"
#include "Styling/AppStyle.h"
#include "UObject/Class.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SBaseReplicationStreamEditor"

namespace UE::ConcertSharedSlate
{
	void SBaseReplicationStreamEditor::Construct(
		const FArguments& InArgs,
		const TSharedRef<IEditableReplicationStreamModel>& InPropertiesModel,
		const TSharedRef<IObjectSelectionSourceModel>& InObjectSelectionSource,
		const TSharedRef<IPropertySourceProcessor>& InPropertySelectionSource)
	{
		ObjectSelectionSource = InObjectSelectionSource;
		PropertySelectionSource = InPropertySelectionSource;
		ObjectHierarchy = InArgs._ObjectHierarchy;
		
		EditablePropertiesModel = InPropertiesModel;
		EditablePropertiesModel->OnObjectsChanged().AddSP(this, &SBaseReplicationStreamEditor::OnObjectsChanged);
		EditablePropertiesModel->OnPropertiesChanged().AddSP(this, &SBaseReplicationStreamEditor::OnPropertiesChanged);

		IsEditingEnabledAttribute = InArgs._IsEditingEnabled;
		EditingDisabledToolTipTextAttribute = InArgs._EditingDisabledToolTipText;
		OnExtendObjectsContextMenuDelegate = InArgs._OnExtendObjectsContextMenu;
		OnPreAddSelectedObjectsDelegate = InArgs._OnPreAddSelectedObjectsDelegate;
		OnPostAddSelectedObjectsDelegate = InArgs._OnPostAddSelectedObjectsDelegate;
		
		ChildSlot
		[
			SAssignNew(ReplicationViewer, SReplicationStreamViewer, EditablePropertiesModel.ToSharedRef())
				.PropertyAssignmentView(InArgs._PropertyAssignmentView)
				.ObjectColumns(InArgs._ObjectColumns)
				.PrimaryObjectSort(InArgs._PrimaryObjectSort)
				.SecondaryObjectSort(InArgs._SecondaryObjectSort)
				.ObjectHierarchy(InArgs._ObjectHierarchy)
				.NameModel(InArgs._NameModel)
				.OnDeleteObjects(this, &SBaseReplicationStreamEditor::OnDeleteObjects)
				.OnObjectsContextMenuOpening(this, &SBaseReplicationStreamEditor::OnObjectsContextMenuOpening)
				.ShouldDisplayObject(InArgs._ShouldDisplayObject)
				.LeftOfObjectSearchBar()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						BuildRootAddObjectWidgets()
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						InArgs._LeftOfObjectSearchBar.Widget
					]
				]
				.RightOfObjectSearchBar() [ InArgs._RightOfObjectSearchBar.Widget ]
				.GetHoveredRowContent_Lambda(
					[MakeOverlay = InArgs._MakeObjectRowOverlayWidget, Alignment = InArgs._ObjectOverlayAlignment]
					(const TSharedPtr<FReplicatedObjectData>& Data)
					{
						const TSharedRef<SWidget> OverlayWidget = MakeOverlay.IsBound() ? MakeOverlay.Execute(*Data) : SNullWidget::NullWidget;
						return FHoverRowContent{ OverlayWidget, Alignment };
					})
				.NoOutlinerObjects(LOCTEXT("NoObjects", "Add objects to replicate"))
				.WrapOutliner(InArgs._WrapOutliner)
		];
	}

	SBaseReplicationStreamEditor::~SBaseReplicationStreamEditor()
	{
		EditablePropertiesModel->OnObjectsChanged().RemoveAll(this);
		EditablePropertiesModel->OnPropertiesChanged().RemoveAll(this);
	}

	void SBaseReplicationStreamEditor::Refresh()
	{
		ReplicationViewer->Refresh();
	}

	void SBaseReplicationStreamEditor::RequestObjectColumnResort(const FName& ColumnId)
	{
		ReplicationViewer->RequestObjectColumnResort(ColumnId);
	}

	void SBaseReplicationStreamEditor::RequestPropertyColumnResort(const FName& ColumnId)
	{
		ReplicationViewer->RequestPropertyColumnResort(ColumnId);
	}

	void SBaseReplicationStreamEditor::SetSelectedObjects(TConstArrayView<TSoftObjectPtr<>> Objects)
	{
		ReplicationViewer->SetSelectedObjects(Objects);
	}

	TArray<TSoftObjectPtr<>> SBaseReplicationStreamEditor::GetSelectedObjects() const
	{
		return ReplicationViewer->GetSelectedObjects();
	}

	bool SBaseReplicationStreamEditor::IsEditingDisabled() const
	{
		return (IsEditingEnabledAttribute.IsBound() || IsEditingEnabledAttribute.IsSet())
			&& !IsEditingEnabledAttribute.Get();
	}

	FText SBaseReplicationStreamEditor::GetEditingDisabledText() const
	{
		return (EditingDisabledToolTipTextAttribute.IsBound() || EditingDisabledToolTipTextAttribute.IsSet())
			? EditingDisabledToolTipTextAttribute.Get()
			: FText::GetEmpty();
	}

	void SBaseReplicationStreamEditor::OnObjectsChanged(TConstArrayView<UObject*> AddedObjects, TConstArrayView<FSoftObjectPath> RemovedObjects, EReplicatedObjectChangeReason ChangeReason)
	{
		ReplicationViewer->RequestObjectDataRefresh();
			
		// Newly added objects should be automatically selected
		if (!AddedObjects.IsEmpty())
		{
			TArray<TSoftObjectPtr<>> TopLevelObjects;
			
			// Goal: select an object so the property view immediately shows properties for some object.
			// Problem: if objects have different classes, property view will be empty (incompatible class)
			// Solution: Select the highest object in the hierarchy, which is usually an actor. Users usually select an actor to add in the "Add" combo button so this also makes intuitive sense.
			// Caveat: There may be multiple hierarchies (e.g. if multiple actors were added). This is a very seldom case though: too bad.
			Algo::TransformIf(AddedObjects, TopLevelObjects,
				[this, &AddedObjects](UObject* Object)
				{
					const TOptional<IObjectHierarchyModel::FParentInfo> ParentInfo = ObjectHierarchy ? ObjectHierarchy->GetParentInfo(Object) : TOptional<IObjectHierarchyModel::FParentInfo>{};
					const bool bIsTopOfHierarchy = !ParentInfo || !AddedObjects.ContainsByPredicate([&ParentInfo](UObject* AddedObject){ return AddedObject == ParentInfo->Parent; });
					return bIsTopOfHierarchy;
				},
				[](UObject* Object){ return Object; });
			ReplicationViewer->SelectObjects(TopLevelObjects);

			// Expand the hierarchy for all added objects for easier editing
			constexpr bool bRecursive = true;
			ReplicationViewer->ExpandObjects(TopLevelObjects, bRecursive);
		}
	}

	void SBaseReplicationStreamEditor::OnPropertiesChanged()
	{
		ReplicationViewer->RequestPropertyDataRefresh();
	}

	TSharedRef<SWidget> SBaseReplicationStreamEditor::BuildRootAddObjectWidgets()
	{
		using namespace ConcertSharedSlate;
		
		const TSharedRef<SHorizontalBox> Root = SNew(SHorizontalBox);
		const FSourceModelBuilders<FSelectableObjectInfo>::FItemPickerArgs Args = MakeObjectSourceBuilderArgs();
		for (const TSourceSelectionCategory<FSelectableObjectInfo>& Category : ObjectSelectionSource->GetRootSources())
		{
			Root->AddSlot()
				.AutoWidth()
				[
					FSourceModelBuilders<FSelectableObjectInfo>::BuildCategory(Category, Args)	
				];
		}
		return Root;
	}

	void SBaseReplicationStreamEditor::OnObjectsSelectedForAdding(TArray<FSelectableObjectInfo> ObjectsToAdd)
	{
		TArray<UObject*> Objects;
		Algo::TransformIf(
			ObjectsToAdd,
			Objects,
			[](const FSelectableObjectInfo& SelectableObject) { return SelectableObject.Object.IsValid(); },
			[](const FSelectableObjectInfo& SelectableObject){ return SelectableObject.Object.Get(); }
			);

		TGuardValue<bool> GuardObjectSelection(bIsAddingFromSelection, true);
		OnPreAddSelectedObjectsDelegate.ExecuteIfBound(ObjectsToAdd);
		EditablePropertiesModel->AddObjects(Objects);
		OnPostAddSelectedObjectsDelegate.ExecuteIfBound(ObjectsToAdd);
	}

	void SBaseReplicationStreamEditor::OnDeleteObjects(const TArray<TSharedPtr<FReplicatedObjectData>>& ObjectsToDelete) const
	{
		if (IsEditingDisabled())
		{
			return;
		}
		
		TArray<FSoftObjectPath> DeleteObjectPaths;
		Algo::Transform(ObjectsToDelete, DeleteObjectPaths, [](const TSharedPtr<FReplicatedObjectData>& Data) { return Data->GetObjectPath(); });

		// We want to delete children not listed in the outliner, such as components and other subobjects
		TArray<FSoftObjectPath> ObjectAndChildren;
		EditablePropertiesModel->ForEachReplicatedObject([&ObjectsToDelete, &ObjectAndChildren](const FSoftObjectPath& ReplicatedObject)
		{
			const bool bIsChildOfDeletedObject = Algo::AnyOf(ObjectsToDelete, [&ReplicatedObject](const TSharedPtr<FReplicatedObjectData>& ObjectData)
			{
				// ReplicatedObject is a child of a deleted object if the path before it contains one of the deleted objects
				return ReplicatedObject.ToString().Contains(ObjectData->GetObjectPath().ToString());
			});
			if (bIsChildOfDeletedObject)
			{
				ObjectAndChildren.Add(ReplicatedObject);
			}
			return EBreakBehavior::Continue;
		});
		
		EditablePropertiesModel->RemoveObjects(ObjectAndChildren);
	}

	TSharedPtr<SWidget> SBaseReplicationStreamEditor::OnObjectsContextMenuOpening()
	{
		FMenuBuilder MenuBuilder(false, nullptr);
		
		AddObjectSourceContextMenuOptions(MenuBuilder);
		MenuBuilder.AddMenuEntry(
			LOCTEXT("DeleteItems", "Delete"),
			TAttribute<FText>::CreateLambda([this](){ return GetEditingDisabledText(); }),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SBaseReplicationStreamEditor::OnDeleteObjects_PassByValue, ReplicationViewer->GetSelectedObjectItems()),
				FCanExecuteAction::CreateLambda([this]() { return !IsEditingDisabled(); })
				),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		TArray<TSoftObjectPtr<>> SelectedObjects;
		Algo::Transform(ReplicationViewer->GetSelectedObjectItems(), SelectedObjects, [](const TSharedPtr<FReplicatedObjectData>& Data){ return Data->GetObjectPtr(); });
		OnExtendObjectsContextMenuDelegate.ExecuteIfBound(MenuBuilder, SelectedObjects);
		
		return MenuBuilder.MakeWidget();
	}

	void SBaseReplicationStreamEditor::AddObjectSourceContextMenuOptions(FMenuBuilder& MenuBuilder)
	{
		using namespace ConcertSharedSlate;
		
		// Context menu generation is only supported for single items
		const TArray<TSharedPtr<FReplicatedObjectData>> SelectedObjects = ReplicationViewer->GetSelectedObjectItems();
		if (SelectedObjects.Num() != 1)
		{
			return;
		}
		
		const TArray<TSharedRef<IObjectSourceModel>> ContextMenuOptions = ObjectSelectionSource->GetContextMenuOptions(SelectedObjects[0]->GetObjectPath());
		if (ContextMenuOptions.IsEmpty())
		{
			return;
		}
		
		const FSourceModelBuilders<FSelectableObjectInfo>::FItemPickerArgs Args = MakeObjectSourceBuilderArgs();
		for (const TSharedRef<IObjectSourceModel>& SourceModel : ContextMenuOptions)
		{
			bool bCanAddAnyObject = false; 
			SourceModel->EnumerateSelectableItems([this, &bCanAddAnyObject](const FSelectableObjectInfo& SelectableOption) mutable
			{
				bCanAddAnyObject |= !EditablePropertiesModel->ContainsObjects({ SelectableOption.Object.Get() });
				return bCanAddAnyObject ? EBreakBehavior::Break : EBreakBehavior::Continue;
			});

			// Skip showing context menu options which will not add anything new.
			if (bCanAddAnyObject)
			{
				FSourceModelBuilders<FSelectableObjectInfo>::AddOptionToMenu(SourceModel, Args, MenuBuilder);
			}
		}
		
		MenuBuilder.AddSeparator();
	}

	ConcertSharedSlate::FSourceModelBuilders<FSelectableObjectInfo>::FItemPickerArgs SBaseReplicationStreamEditor::MakeObjectSourceBuilderArgs()
	{
		using namespace ConcertSharedSlate;
		using FBuilderDelegates = FSourceModelBuilders<FSelectableObjectInfo>;
		
		return FSourceModelBuilders<FSelectableObjectInfo>::FItemPickerArgs
		{
			FBuilderDelegates::FOnItemsSelected::CreateSP(this, &SBaseReplicationStreamEditor::OnObjectsSelectedForAdding),
			FBuilderDelegates::FGetItemDisplayString::CreateLambda([](const FSelectableObjectInfo& Item)
			{
				return Item.Object.IsValid() ? DisplayUtils::GetObjectDisplayString(*Item.Object.Get()) : TEXT("");
			}),
			FBuilderDelegates::FGetItemIcon::CreateLambda([](const FSelectableObjectInfo& Item)
			{
				return Item.Object.IsValid() ? DisplayUtils::GetObjectIcon(*Item.Object.Get()) : FSlateIcon();
			}),
			FBuilderDelegates::FIsItemSelected::CreateLambda([this](const FSelectableObjectInfo& Item)
			{
				const UObject* Object = Item.Object.Get();
				return !Object || ReplicationViewer->IsDisplayedInTopView(Object);
			}),
			TAttribute<bool>::CreateLambda([this]() { return !IsEditingDisabled(); }),
			TAttribute<FText>::CreateLambda([this]() { return GetEditingDisabledText(); }),
			EItemPickerFlags::DisplayOptionListInline
		}; 
	}
}

#undef LOCTEXT_NAMESPACE