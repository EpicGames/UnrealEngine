// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Model/Item/SourceModelBuilders.h"
#include "Replication/Editor/View/IReplicationStreamEditor.h"
#include "Replication/Editor/View/Column/IObjectTreeColumn.h"
#include "Replication/Editor/View/Column/ReplicationColumnsUtils.h"
#include "Replication/Utils/ReplicationWidgetDelegates.h"

#include "Widgets/DeclarativeSyntaxSupport.h"

class SHorizontalBox;
struct FConcertPropertyChain;
struct FConcertStreamObjectAutoBindingRules;

namespace UE::ConcertSharedSlate
{
	class FFakeObjectToPropertiesEditorModel;
	class FReplicatedObjectData;
	class FPropertyData;
	class IEditableReplicationStreamModel;
	class IObjectNameModel;
	class IObjectSelectionSourceModel;
	class IPropertyAssignmentView;
	class IPropertySourceProcessor;
	class IObjectHierarchyModel;
	class SReplicationStreamViewer;
	
	enum class EReplicatedObjectChangeReason : uint8;

	struct FHoverRowContent;
	struct FSelectableObjectInfo;
	
	/**
	 * Extends SObjectToPropertyView with common functionality for editing (with an IEditableReplicationStreamModel):
	 * - Pressing delete calls IEditableReplicationStreamModel::RemoveObjects
	 * - A combo box to left of the object search bar for adding new top level objects
	 * - Default context menu items for top-level objects, such as deleting or adding more based right-clicked object
	 * - Wraps the IEditableReplicationStreamModel with FFakeObjectToPropertiesEditorModel so the SObjectToPropertyView:
	 *		- displays only top-level objects in the outliner (i.e. excludes components)
	 *		- displays all properties in the property view instead of just those registered in the model
	 *
	 * This is a base widget and is not fully functional, yet. You must inject additional UI to support adding objects.
	 * @see SDefaultReplicationStreamEditor for an example: it adds a Replicates? checkbox to the property rows.
	 *
	 * This widget assumes that the IReplicationSubobjectView::GetSelectedObjects reports subobjects that are
	 * not in the model yet. Such objects are added to the model.
	 */
	class SBaseReplicationStreamEditor : public IReplicationStreamEditor
	{
	public:
		
		SLATE_BEGIN_ARGS(SBaseReplicationStreamEditor)
		{}
			/** In the lower half of the editor, this view presents the properties associated with the object that is currently selected in the upper part of the view. */
			SLATE_ARGUMENT(TSharedPtr<IPropertyAssignmentView>, PropertyAssignmentView)
		
			/** Additional columns to add to the object view */
			SLATE_ARGUMENT(TArray<FObjectColumnEntry>, ObjectColumns)
			/** Initial primary sort to set. */
			SLATE_ARGUMENT(FColumnSortInfo, PrimaryObjectSort)
			/** Initial secondary sort to set. */
			SLATE_ARGUMENT(FColumnSortInfo, SecondaryObjectSort)
		
			/** Optional. If set, this determines the children nested under the root objects. */
			SLATE_ARGUMENT(TSharedPtr<IObjectHierarchyModel>, ObjectHierarchy)
			/** Optional. If set, this determines the display text for objects. */
			SLATE_ARGUMENT(TSharedPtr<IObjectNameModel>, NameModel)

			/** Called to generate the context menu for objects. This extends the options already generated by this widget. */
			SLATE_EVENT(FExtendObjectMenu, OnExtendObjectsContextMenu)
			
			/** Called just before the passed objects are added to the stream model. Called in response to the user selecting these objects from the combo button. */
			SLATE_EVENT(FSelectObjectsFromComboButton, OnPreAddSelectedObjectsDelegate)
			/** Called just after the passed objects have been added to the stream model. Called in response to the user selecting these objects from the combo button. */
			SLATE_EVENT(FSelectObjectsFromComboButton, OnPostAddSelectedObjectsDelegate)
			
			/** Optional. Whether a given object should be displayed. If this returns false on an object, none of its children will be shown either. */
			SLATE_EVENT(FShouldDisplayObject, ShouldDisplayObject)

			/** Optional. A widget that should be overlayed on object rows. */
			SLATE_EVENT(FMakeObjectRowOverlayWidget, MakeObjectRowOverlayWidget)
			/** Optional. Controls how the widgets created by MakeObjectRowOverlayWidget are supposed to be aligned. */
			SLATE_ARGUMENT(EHorizontalAlignment, ObjectOverlayAlignment)

			/** Optional widget to add to the left of the object list search bar. */
			SLATE_NAMED_SLOT(FArguments, LeftOfObjectSearchBar)
			/** Optional widget to add to the right of the object list search bar. */
			SLATE_NAMED_SLOT(FArguments, RightOfObjectSearchBar)

			/** Optional. Determines whether all UI for changing the model should be disabled. */
			SLATE_ATTRIBUTE(bool, IsEditingEnabled)
			/** Optional. Whenever IsEditingEnabled returns true, this tooltip is displayed for relevant, disabled UI. */
			SLATE_ATTRIBUTE(FText, EditingDisabledToolTipText)
			
			/** Optional. Returns new widget that wraps the outliner widget, which displays the replicated objects. Could return e.g. an SOverlay or SDropTarget. */
			SLATE_EVENT(FWrapOutlinerWidget, WrapOutliner)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs,
           const TSharedRef<IEditableReplicationStreamModel>& InPropertiesModel,
           const TSharedRef<IObjectSelectionSourceModel>& InObjectSelectionSource,
           const TSharedRef<IPropertySourceProcessor>& InPropertySelectionSource
		);
		virtual ~SBaseReplicationStreamEditor() override;

		//~ Begin IReplicationStreamEditor Interface
		virtual void Refresh() override;
		virtual void RequestObjectColumnResort(const FName& ColumnId) override;
		virtual void RequestPropertyColumnResort(const FName& ColumnId) override;
		virtual void SetSelectedObjects(TConstArrayView<TSoftObjectPtr<>> Objects) override;
		virtual TArray<TSoftObjectPtr<>> GetSelectedObjects() const override;
		//~ End IReplicationStreamEditor Interface

	private:

		/** Displays the objects and properties */
		TSharedPtr<SReplicationStreamViewer> ReplicationViewer;

		/** For reading and writting to the edited asset */
		TSharedPtr<IEditableReplicationStreamModel> EditablePropertiesModel;
		/** Can be null. If set, adds all subobjects to the top level view. */
		TSharedPtr<IObjectHierarchyModel> ObjectHierarchy;
		
		/** For deciding which objects can be added to EditablePropertiesModel. */
		TSharedPtr<IObjectSelectionSourceModel> ObjectSelectionSource;
		/** For deciding which properties can be added to EditablePropertiesModel. */
		TSharedPtr<IPropertySourceProcessor> PropertySelectionSource;

		/** Optional. Determines whether all UI for changing the model should be disabled. */
		TAttribute<bool> IsEditingEnabledAttribute;
		/** Optional. Whenever IsEditingEnabled returns true, this tooltip is displayed for relevant, disabled UI. */
		TAttribute<FText> EditingDisabledToolTipTextAttribute;
		
		/** Generates additional entries */
		FExtendObjectMenu OnExtendObjectsContextMenuDelegate;
		/** Called just before the passed objects are added to the stream model. Called in response to the user selecting these objects from the combo button. */
		FSelectObjectsFromComboButton OnPreAddSelectedObjectsDelegate;
		/** Called just after the passed objects have been added to the stream model. Called in response to the user selecting these objects from the combo button. */
		FSelectObjectsFromComboButton OnPostAddSelectedObjectsDelegate;

		/**
		 * Set when the combo button is adding an object.
		 * This is so AutoAddObjectsAndPropertiesFromSettings only adds when an object is added via the combo button.
		 * Note that AutoAddObjectsAndPropertiesFromSettings must be called in OnObjectsChanged so it becomes part of the transaction.
		 */
		bool bIsAddingFromSelection = false;

		bool IsEditingDisabled() const;
		FText GetEditingDisabledText() const;
		
		void OnObjectsChanged(TConstArrayView<UObject*> AddedObjects, TConstArrayView<FSoftObjectPath> RemovedObjects, EReplicatedObjectChangeReason ChangeReason);
		void OnPropertiesChanged();

		// Customizing left objects search bar
		TSharedRef<SWidget> BuildRootAddObjectWidgets();
		void OnObjectsSelectedForAdding(TArray<FSelectableObjectInfo> ObjectsToAdd);

		// Respond to object editing events
		void OnDeleteObjects_PassByValue(TArray<TSharedPtr<FReplicatedObjectData>> CopiedObjectsToDelete) const { OnDeleteObjects(CopiedObjectsToDelete); }
		void OnDeleteObjects(const TArray<TSharedPtr<FReplicatedObjectData>>& ObjectsToDelete) const;
		TSharedPtr<SWidget> OnObjectsContextMenuOpening();
		void AddObjectSourceContextMenuOptions(FMenuBuilder& MenuBuilder);
		
		// Utils for building item source widgets
		FSourceModelBuilders<FSelectableObjectInfo>::FItemPickerArgs MakeObjectSourceBuilderArgs();
	};
}
