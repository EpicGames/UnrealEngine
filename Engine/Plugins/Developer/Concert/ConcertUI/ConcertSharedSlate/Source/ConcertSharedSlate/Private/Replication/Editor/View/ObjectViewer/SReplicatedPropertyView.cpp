// Copyright Epic Games, Inc. All Rights Reserved.

#include "SReplicatedPropertyView.h"

#include "Replication/Editor/Model/IReplicationStreamModel.h"
#include "Replication/Editor/Model/Data/ReplicatedObjectData.h"
#include "Trace/ConcertTrace.h"

#include "Algo/AllOf.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SReplicatedPropertyView"

namespace UE::ConcertSharedSlate
{
	void SReplicatedPropertyView::Construct(const FArguments& InArgs, TSharedRef<IPropertyAssignmentView> InPropertyAssignmentView, TSharedRef<IReplicationStreamModel> InPropertiesModel)
	{
		PropertyAssignmentView = MoveTemp(InPropertyAssignmentView);
		PropertiesModel = MoveTemp(InPropertiesModel);
		
		GetObjectClassDelegate = InArgs._GetObjectClass;
		
		ChildSlot
		[
			CreatePropertiesView(InArgs)
		];
	}
	
	void SReplicatedPropertyView::RefreshPropertyData(const TArray<TSoftObjectPtr<>>& SelectedObjects)
	{
		SCOPED_CONCERT_TRACE(RefreshPropertyData);
		if (SelectedObjects.IsEmpty())
		{
			SetPropertyContent(EReplicatedPropertyContent::NoSelection);
			return;
		}

		// Technically, the classes just need to be compatible with each other... but it is easier to just allow the same class.
		const TOptional<FSoftClassPath> SharedClass = GetClassForPropertiesFromSelection(SelectedObjects);
		if (!SharedClass)
		{
			SetPropertyContent(EReplicatedPropertyContent::SelectionTooBig);
			return;
		}
		
		PropertyAssignmentView->RefreshData(SelectedObjects, *PropertiesModel);
		SetPropertyContent(EReplicatedPropertyContent::Properties);
	}

	TSharedRef<SWidget> SReplicatedPropertyView::CreatePropertiesView(const FArguments& InArgs)
	{
		return SAssignNew(PropertyContent, SWidgetSwitcher)
			// Make sure the slots are coherent with the order of EReplicatedPropertyContent!
			.WidgetIndex(static_cast<int32>(EReplicatedPropertyContent::NoSelection))
			
			// EReplicatedPropertyContent::Properties
			+SWidgetSwitcher::Slot()
			[
				PropertyAssignmentView->GetWidget()
			]
			
			// EReplicatedPropertyContent::NoSelection
			+SWidgetSwitcher::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NoPropertyEditedObjects", "Select an object to see selected properties"))
			]
			
			// EReplicatedPropertyContent::SelectionTooBig
			+SWidgetSwitcher::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("SelectionTooBig", "Select objects of the same type type to see selected properties"))
			];
	}
	
	TOptional<FSoftClassPath> SReplicatedPropertyView::GetClassForPropertiesFromSelection(const TArray<TSoftObjectPtr<>>& Objects) const
	{
		FSoftClassPath SharedClass;
		const bool bAllHaveSameClass = Algo::AllOf(Objects, [this, &SharedClass](const TSoftObjectPtr<>& Object)
		{
			const FSoftClassPath ObjectClass = GetObjectClass(Object);
			SharedClass = SharedClass.IsValid() ? SharedClass : ObjectClass;
			return ObjectClass == SharedClass;
		});
		return bAllHaveSameClass ? SharedClass : TOptional<FSoftClassPath>{};
	}

	void SReplicatedPropertyView::SetPropertyContent(EReplicatedPropertyContent Content) const
	{
		PropertyContent->SetActiveWidgetIndex(static_cast<int32>(Content));
	}
}

#undef LOCTEXT_NAMESPACE