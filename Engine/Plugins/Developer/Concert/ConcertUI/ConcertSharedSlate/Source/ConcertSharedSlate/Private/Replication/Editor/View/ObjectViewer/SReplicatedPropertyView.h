// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Utils/ReplicationWidgetDelegates.h"
#include "Replication/Editor/View/Column/SelectionViewerColumns.h"
#include "Replication/Editor/View/IPropertyAssignmentView.h"

#include "Misc/Optional.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SWidgetSwitcher;

namespace UE::ConcertSharedSlate
{
	/**
	 * Determines which properties are to be displayed based on an IReplicationStreamModel.
	 * Uses a property tree for displaying. If no properties are displayed, this widget displays a message instead, e.g. to select an object.
	 */
	class SReplicatedPropertyView : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SReplicatedPropertyView)
		{}
			/** Gets the class for the object since the object may not be in the model. */
			SLATE_EVENT(FGetObjectClass, GetObjectClass)
		
			/** Optional. If set, this determines the display text for objects. */
			SLATE_ARGUMENT(TSharedPtr<IObjectNameModel>, NameModel)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedRef<IPropertyAssignmentView> InPropertyAssignmentView, TSharedRef<IReplicationStreamModel> InPropertiesModel);

		/** Updates the displayed properties */
		void RefreshPropertyData(const TArray<TSoftObjectPtr<>>& SelectedObjects);
		/** Requests that the given column be resorted, if it currently affects the row sorting. */
		void RequestResortForColumn(const FName& ColumnId) const { PropertyAssignmentView->RequestResortForColumn(ColumnId); }

	private:
		
		/** In the lower half of the editor, this view presents the properties associated with the object that is currently selected in the upper part of the view. */
		TSharedPtr<IPropertyAssignmentView> PropertyAssignmentView;
		
		/** The model this view is visualizing. */
		TSharedPtr<IReplicationStreamModel> PropertiesModel;
		
		enum class EReplicatedPropertyContent
		{
			/** Shows the properties */
			Properties,
			/** Prompts: "Select an object to see selected properties" */
			NoSelection,
			/** Prompts: "Select objects of the same type type to see selected properties" */
			SelectionTooBig
		};
		/** Determines the content displayed for PropertyArea. */
		TSharedPtr<SWidgetSwitcher> PropertyContent;
		
		/** Gets the class for the object since the object may not be in the model. */
		FGetObjectClass GetObjectClassDelegate;

		TSharedRef<SWidget> CreatePropertiesView(const FArguments& InArgs);
		
		/** Given the selected objects, determines whether they all have the same class and returns it if so. */
		TOptional<FSoftClassPath> GetClassForPropertiesFromSelection(const TArray<TSoftObjectPtr<>>& Objects) const;
		/** Sets how to display this widget */
		void SetPropertyContent(EReplicatedPropertyContent Content) const;

		FSoftClassPath GetObjectClass(const TSoftObjectPtr<>& Object) const { return GetObjectClassDelegate.Execute(Object); }
	};
}


