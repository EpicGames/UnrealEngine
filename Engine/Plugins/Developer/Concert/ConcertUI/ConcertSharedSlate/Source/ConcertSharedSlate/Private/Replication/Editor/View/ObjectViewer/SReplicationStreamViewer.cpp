// Copyright Epic Games, Inc. All Rights Reserved.

#include "SReplicationStreamViewer.h"

#include "ConcertFrontendUtils.h"
#include "Replication/Editor/Model/IReplicationStreamModel.h"
#include "Replication/Editor/Model/Object/IObjectHierarchyModel.h"
#include "Replication/Editor/Model/Data/ReplicatedObjectData.h"
#include "Replication/Editor/View/Column/ObjectColumnAdapter.h"
#include "Replication/Editor/View/Column/SelectionViewerColumns.h"
#include "Replication/Editor/View/Property/SPropertyTreeView.h"
#include "Misc/ObjectUtils.h"
#include "SReplicatedPropertyView.h"
#include "Trace/ConcertTrace.h"

#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#if WITH_EDITOR
#include "UObject/Class.h"
#include "UObject/Object.h"
#endif

#define LOCTEXT_NAMESPACE "SObjectToPropertyView"

namespace UE::ConcertSharedSlate
{
	namespace Private
	{
		static TSharedRef<FReplicatedObjectData> AllocateObjectData(FSoftObjectPath ObjectPath)
		{
			return MakeShared<FReplicatedObjectData>(MoveTemp(ObjectPath));
		}

		static TSharedRef<FReplicatedObjectData> AllocateObjectData(TSoftObjectPtr<> Object)
		{
			return MakeShared<FReplicatedObjectData>(MoveTemp(Object));
		}
	}
	
	void SReplicationStreamViewer::Construct(const FArguments& InArgs, const TSharedRef<IReplicationStreamModel>& InPropertiesModel)
	{
		PropertiesModel = InPropertiesModel;
		ObjectHierarchy = InArgs._ObjectHierarchy;
		NameModel = InArgs._NameModel;
		ShouldDisplayObjectDelegate = InArgs._ShouldDisplayObject;
		
		ChildSlot
		[
			CreateContentWidget(InArgs)
		];

		Refresh();
		PropertyArea->SetExpanded(true);
	}

	void SReplicationStreamViewer::Refresh()
	{
		RequestObjectDataRefresh();
		RequestPropertyDataRefresh();
	}

	void SReplicationStreamViewer::RequestObjectColumnResort(const FName& ColumnId)
	{
		ReplicatedObjects->RequestResortForColumn(ColumnId);
	}

	void SReplicationStreamViewer::RequestPropertyColumnResort(const FName& ColumnId)
	{
		PropertySection->RequestResortForColumn(ColumnId);
	}

	void SReplicationStreamViewer::SetSelectedObjects(TConstArrayView<TSoftObjectPtr<>> Objects)
	{
		// Do the selection at the end of the tick in case anything still needs updating this tick.
		SelectObjects(Objects, true);
	}

	TArray<TSoftObjectPtr<>> SReplicationStreamViewer::GetSelectedObjects() const
	{
		TArray<TSoftObjectPtr<>> Result;
		Algo::Transform(GetSelectedObjectItems(), Result, [](const TSharedPtr<FReplicatedObjectData>& Item)
		{
			return Item->GetObjectPtr();
		});
		return Result;
	}

	void SReplicationStreamViewer::SelectObjects(TConstArrayView<TSoftObjectPtr<>> Objects, bool bAtEndOfTick)
	{
		SCOPED_CONCERT_TRACE(SelectObjects);
		
		if (bHasRequestedObjectRefresh || bAtEndOfTick)
		{
			PendingToSelect = Objects;
			return;
		}
		
		TArray<TSharedPtr<FReplicatedObjectData>> NewSelectedItems; 
		Algo::TransformIf(AllObjectRowData, NewSelectedItems, [this, &Objects](const TSharedPtr<FReplicatedObjectData>& ObjectData)
			{
				return Objects.Contains(ObjectData->GetObjectPtr()) && CanDisplayObject(ObjectData->GetObjectPtr());
			},
			[](const TSharedPtr<FReplicatedObjectData>& ObjectData){ return ObjectData; }
		);
		if (!NewSelectedItems.IsEmpty())
		{
			ReplicatedObjects->SetSelectedItems(NewSelectedItems, true);
		}
	}

	void SReplicationStreamViewer::ExpandObjects(TConstArrayView<TSoftObjectPtr<>> Objects, bool bRecursive, bool bAtEndOfTick)
	{
		SCOPED_CONCERT_TRACE(ExpandObjects);
		
		if (Objects.IsEmpty())
		{
			return;
		}

		if (bHasRequestedObjectRefresh || bAtEndOfTick)
		{
			PendingToExpand = Objects;
			bPendingExpandRecursively = bRecursive;
			return;
		}
		
		TArray<TSharedPtr<FReplicatedObjectData>> ItemsToExpand;
		ItemsToExpand.Reserve(Objects.Num());
		for (const TSoftObjectPtr<>& Path : Objects)
		{
			if (const TSharedPtr<FReplicatedObjectData>* Item = PathToObjectDataCache.Find(Path.GetUniqueID()))
			{
				ItemsToExpand.Add(*Item);
			}
			
			if (bRecursive && ObjectHierarchy)
			{
				ObjectHierarchy->ForEachChildRecursive(Path, [this, &ItemsToExpand](const TSoftObjectPtr<>&, const TSoftObjectPtr<>& ChildObject, EChildRelationship)
				{
					if (const TSharedPtr<FReplicatedObjectData>* Item = PathToObjectDataCache.Find(ChildObject.GetUniqueID()))
					{
						ItemsToExpand.Add(*Item);
					}
					return EBreakBehavior::Continue;
				});
			}
		}

		if (!ItemsToExpand.IsEmpty())
		{
			ReplicatedObjects->SetExpandedItems(ItemsToExpand, true);
		}
	}

	bool SReplicationStreamViewer::IsDisplayedInTopView(const FSoftObjectPath& Object) const
	{
		const bool bIsContainedActor = PropertiesModel->ContainsObjects({ Object } );
		const bool bIsContainedSubobject = PropertiesModel->AnyOfSubobjects(Object, [this](const FSoftObjectPath& SubobjectPath)
		{
			return PropertiesModel->ContainsObjects({ SubobjectPath } );
		});
		return bIsContainedActor || bIsContainedSubobject;
	}

	TArray<TSharedPtr<FReplicatedObjectData>> SReplicationStreamViewer::GetSelectedObjectItems() const
	{
		TArray<TSharedPtr<FReplicatedObjectData>> SelectedItems = ReplicatedObjects->GetSelectedItems();
		// Items may have been removed this tick. However, selected items may not have been updated yet because STreeView processes item changes at the end of tick. 
		SelectedItems.SetNum(Algo::RemoveIf(SelectedItems, [this](const TSharedPtr<FReplicatedObjectData>& ObjectData)
		{
			const FSoftObjectPath& ObjectPath = ObjectData->GetObjectPath();
			const bool bIsInModel = PropertiesModel->ContainsObjects({ ObjectPath });
			
			const TOptional<FSoftObjectPath> OwningActor = ConcertSyncCore::GetActorOf(ObjectPath);
			// When displaying object from local machine...
			// ... the "Add Actor" button has added an actor without properties to the model; however objects without assigned properties are not transmitted to server.
			const bool bContainsOwningActor = OwningActor.IsSet() && PropertiesModel->ContainsObjects({ *OwningActor });

			// When displaying object from remote machine...
			// ... we only see objects with actual properties assigned.
			const bool bIsSubobjectOfActor = OwningActor.IsSet() && ObjectPath.ToString().Contains(OwningActor->ToString());
			// ... if ObjectData is an actor, we must also consider whether any of its subobjects is contained in the model
			bool bContainsAnySubobject = false;
			PropertiesModel->ForEachSubobject(ObjectPath, [&bContainsAnySubobject](const FSoftObjectPath& Child){ bContainsAnySubobject = true; return EBreakBehavior::Break; });
			
			return !bIsInModel && !bContainsOwningActor && !bIsSubobjectOfActor && !bContainsAnySubobject;
		}));
		return SelectedItems;
	}

	void SReplicationStreamViewer::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		SCOPED_CONCERT_TRACE(TickReplicationStreamViewer);
		
		if (bHasRequestedObjectRefresh)
		{
			bHasRequestedObjectRefresh = false;
			bHasRequestedPropertyRefresh = true;
			RefreshObjectData();
		}

		if (bHasRequestedPropertyRefresh)
		{
			bHasRequestedPropertyRefresh = false;
			PropertySection->RefreshPropertyData(
				// If we're about to change the selection, pass in those objects
				PendingToSelect.IsEmpty() ? GetSelectedObjects() : PendingToSelect
				);
		}

		if (!PendingToSelect.IsEmpty())
		{
			SelectObjects(PendingToSelect);
			PendingToSelect.Reset();
		}

		if (!PendingToExpand.IsEmpty())
		{
			ExpandObjects(PendingToExpand, bPendingExpandRecursively);
			PendingToExpand.Reset();
		}
		
		IReplicationStreamViewer::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	}

	TSharedRef<SWidget> SReplicationStreamViewer::CreateContentWidget(const FArguments& InArgs)
	{
		return SNew(SSplitter)
			.Orientation(Orient_Vertical)

			+SSplitter::Slot()
			 .Value(1.f)
			[
				CreateOutlinerSection(InArgs)
			]

			+SSplitter::Slot()
			.SizeRule(TAttribute<SSplitter::ESizeRule>(this, &SReplicationStreamViewer::GetPropertyAreaSizeRule))
			.Value(2.f)
			[
				CreatePropertiesSection(InArgs)
			];
	}

	TSharedRef<SWidget> SReplicationStreamViewer::CreateOutlinerSection(const FArguments& InArgs)
	{
		const FGetObjectClass GetObjectClassDelegate = FGetObjectClass::CreateSP(this, &SReplicationStreamViewer::GetObjectClass);
		
		TArray<FObjectColumnEntry> Columns = InArgs._ObjectColumns;
		Columns.Add(ReplicationColumns::TopLevel::LabelColumn(NameModel.Get(), GetObjectClassDelegate));
		Columns.Add(ReplicationColumns::TopLevel::TypeColumn(GetObjectClassDelegate));
		Columns.Add(ReplicationColumns::TopLevel::NumPropertiesColumn(*PropertiesModel));
		
		const bool bHasNoOutlinerObjectsAttribute = InArgs._NoOutlinerObjects.IsBound() || InArgs._NoOutlinerObjects.IsSet(); 
		const TAttribute<FText> NoObjectsAttribute = bHasNoOutlinerObjectsAttribute ? InArgs._NoOutlinerObjects : LOCTEXT("NoObjects", "No objects to display");

		// Set both primary and secondary in case one is overriden but always use the override.
		const FColumnSortInfo PrimaryObjectSort = InArgs._PrimaryObjectSort.IsValid()
			? InArgs._PrimaryObjectSort
			: FColumnSortInfo{ ReplicationColumns::TopLevel::LabelColumnId, EColumnSortMode::Ascending };
		const FColumnSortInfo SecondaryObjectSort = InArgs._SecondaryObjectSort.IsValid()
			? InArgs._SecondaryObjectSort
			: FColumnSortInfo{ ReplicationColumns::TopLevel::LabelColumnId, EColumnSortMode::Ascending };

		const TSharedRef<SWidget> RightOfSearch = !ObjectHierarchy
			? InArgs._RightOfObjectSearchBar.Widget
			// If the API user specifies an object hierarchy, then display view options for showing the actors' subobjects.
			: [this, &InArgs]()
			{
				ObjectViewOptions.OnDisplaySubobjectsToggled().AddSP(this, &SReplicationStreamViewer::OnSubobjectViewOptionToggled);
				return SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						InArgs._RightOfObjectSearchBar.Widget
					]
				
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						ObjectViewOptions.MakeViewOptionsComboButton()
					];
			}();
		
		TSharedRef<SWidget> Outliner = SAssignNew(ReplicatedObjects, SReplicationTreeView<FReplicatedObjectData>)
			.RootItemsSource(&RootObjectRowData)
			.OnGetChildren(this, &SReplicationStreamViewer::GetObjectRowChildren)
			.OnContextMenuOpening(InArgs._OnObjectsContextMenuOpening)
			.OnDeleteItems(InArgs._OnDeleteObjects)
			.OnSelectionChanged_Lambda([this]()
			{
				RequestPropertyDataRefresh();
			})
			.Columns(FObjectColumnAdapter::Transform(MoveTemp(Columns)))
			.ExpandableColumnLabel(ReplicationColumns::TopLevel::LabelColumnId)
			.PrimarySort(PrimaryObjectSort)
			.SecondarySort(SecondaryObjectSort)
			.SelectionMode(ESelectionMode::Multi)
			.LeftOfSearchBar() [ InArgs._LeftOfObjectSearchBar.Widget ]
			.RightOfSearchBar()
			[
				RightOfSearch
			]
			.NoItemsContent() [ SNew(STextBlock).Text(NoObjectsAttribute) ]
			.GetHoveredRowContent(InArgs._GetHoveredRowContent)
			.RowStyle(FAppStyle::Get(), "TableView.AlternatingRow");

		return InArgs._WrapOutliner.IsBound()
			? InArgs._WrapOutliner.Execute(Outliner)
			: Outliner;
	}

	TSharedRef<SWidget> SReplicationStreamViewer::CreatePropertiesSection(const FArguments& InArgs)
	{
		return SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			.Padding(0.0f)
			[
				SAssignNew(PropertyArea, SExpandableArea)
				.InitiallyCollapsed(true) 
				.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
				.BorderImage_Lambda([this]() { return ConcertFrontendUtils::GetExpandableAreaBorderImage(*PropertyArea); })
				.BodyBorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
				.BodyBorderBackgroundColor(FLinearColor::White)
				.OnAreaExpansionChanged(this, &SReplicationStreamViewer::OnPropertyAreaExpansionChanged)
				.Padding(0.0f)
				.HeaderContent()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ReplicatedProperties", "Properties"))
					.Font(FAppStyle::Get().GetFontStyle("DetailsView.CategoryFontStyle"))
					.ShadowOffset(FVector2D(1.0f, 1.0f))
				]
				.BodyContent()
				[
					SAssignNew(PropertySection, SReplicatedPropertyView, InArgs._PropertyAssignmentView.ToSharedRef(), PropertiesModel.ToSharedRef())
					.GetObjectClass(this, &SReplicationStreamViewer::GetObjectClass)
					.NameModel(InArgs._NameModel)
				]
			];
	}
	
	void SReplicationStreamViewer::RefreshObjectData()
	{
		SCOPED_CONCERT_TRACE(RefreshObjectData);
		
		// Re-using existing instances is tricky: we cannot update the object path in an item because the list view will no detect this change;
		// list view only looks at the shared ptr address. So the UI will not be refreshed. Since the number of items will be small, just reallocate... 
		AllObjectRowData.Empty();

		// Try to re-use old instances by using the old PathToObjectDataCache. This is also done so the expansion states restore correctly in the tree view.
		TMap<FSoftObjectPath, TSharedPtr<FReplicatedObjectData>> NewPathToObjectDataCache;
		
		// Do a complete refresh.
		// Complete refresh is acceptable because the list is updated infrequently and typically small < 500 items.
		// An alternative would be to change RefreshObjectData to be called with two variables ObjectsAdded and ObjectsRemoved.
		IterateDisplayableObjects([this, &NewPathToObjectDataCache](const FSoftObjectPath& ObjectPath) mutable
		{
			const TSharedPtr<FReplicatedObjectData>* ExistingItem = PathToObjectDataCache.Find(ObjectPath);
			ExistingItem = ExistingItem ? ExistingItem : NewPathToObjectDataCache.Find(ObjectPath);
			const TSharedRef<FReplicatedObjectData> Item = ExistingItem
				? ExistingItem->ToSharedRef()
				: Private::AllocateObjectData(ObjectPath);
			AllObjectRowData.AddUnique(Item);
			NewPathToObjectDataCache.Emplace(ObjectPath, Item);
			
			BuildObjectHierarchyIfNeeded(Item, NewPathToObjectDataCache);
			
			return EBreakBehavior::Continue;
		});

		// Only refresh the tree if it is necessary as it causes us to select stuff in the subobject view
		if (!PathToObjectDataCache.OrderIndependentCompareEqual(NewPathToObjectDataCache))
		{
			// If an item was removed, then NewPathToObjectDataCache does not contain it. 
			PathToObjectDataCache = MoveTemp(NewPathToObjectDataCache);

			// The tree view requires the item source to only contain the root items. Children are discovered via GetObjectRowChildren. We re-use GetObjectRowChildren to remove any non-root nodes.
			BuildRootObjectRowData();
			ReplicatedObjects->RequestRefilter();
		}
	}

	void SReplicationStreamViewer::IterateDisplayableObjects(TFunctionRef<void(const FSoftObjectPath& Object)> Delegate) const
	{
		// Case: RealModel contains only components but not the owning actor.
		// In that case, we want the UI to still show the owning actor.
		// We'll track this with these containers:
		TSet<FSoftObjectPath> AddedActors;
		TSet<FSoftObjectPath> PendingActors;

		PropertiesModel->ForEachReplicatedObject([this, &Delegate, &AddedActors, &PendingActors](const FSoftObjectPath& Object)
		{
			if (const TOptional<FSoftObjectPath> OwningActor = ConcertSyncCore::GetActorOf(Object)
				; OwningActor && CanDisplayObject(*OwningActor))
			{
				PendingActors.Add(*OwningActor);
			}
			if (!CanDisplayObject(Object))
			{
				return EBreakBehavior::Continue;
			}
			
			Delegate(Object);
			if (ConcertSyncCore::IsActor(Object))
			{
				AddedActors.Add(Object);
			}
			
			return EBreakBehavior::Continue;
		});

		// Now determine the actors that are not in PropertiesModel but that need to be shown because its subobjects that are in PropertiesModel
		for (const FSoftObjectPath& PendingActor : PendingActors)
		{
			if (!AddedActors.Contains(PendingActor))
			{
				Delegate(PendingActor);
				AddedActors.Add(PendingActor);
			}
		}
	}

	void SReplicationStreamViewer::BuildRootObjectRowData()
	{
		RootObjectRowData.Empty();
		
		TSet<TSharedPtr<FReplicatedObjectData>> NonRootNodes;
		for (const TSharedPtr<FReplicatedObjectData>& Node : AllObjectRowData)
		{
			if (ConcertSyncCore::IsActor(Node->GetObjectPath()))
			{
				RootObjectRowData.Add(Node);
			}
		}
		
		RootObjectRowData.Sort([](const TSharedPtr<FReplicatedObjectData>& Left, const TSharedPtr<FReplicatedObjectData>& Right)
		{
			return Left->GetObjectPath().GetSubPathString() < Right->GetObjectPath().GetSubPathString();
		});
	}

	void SReplicationStreamViewer::BuildObjectHierarchyIfNeeded(TSharedPtr<FReplicatedObjectData> ReplicatedObjectData, TMap<FSoftObjectPath, TSharedPtr<FReplicatedObjectData>>& NewPathToObjectDataCache)
	{
		// We're are not supposed to display any hierarchy in the outliner if ObjectHierarchy is not set.
		const FSoftObjectPath& ObjectPath = ReplicatedObjectData->GetObjectPath();
		if (!ObjectHierarchy)
		{
			return;
		}

		// Find top level object of ReplicatedObjectData
		const FSoftObjectPath OwningActor = ConcertSyncCore::GetActorOf(ObjectPath).Get(ObjectPath);
		if (!ConcertSyncCore::IsActor(OwningActor))
		{
			return;
		}
		
		// Add all objects that appear in the hierarchy of ReplicatedObjectData
		const auto AddItem = [this, &NewPathToObjectDataCache](const TSoftObjectPtr<>& Object)
		{
			const FSoftObjectPath& ObjectPath = Object.GetUniqueID();
			const TSharedPtr<FReplicatedObjectData>* ExistingItem = PathToObjectDataCache.Find(ObjectPath);
			ExistingItem = ExistingItem ? ExistingItem : NewPathToObjectDataCache.Find(ObjectPath);
			const TSharedRef<FReplicatedObjectData> Item = ExistingItem
				? ExistingItem->ToSharedRef()
				: Private::AllocateObjectData(ObjectPath);
			AllObjectRowData.AddUnique(Item);
			NewPathToObjectDataCache.Emplace(ObjectPath, Item);
		};
		
		const TSoftObjectPtr ActorPtr(OwningActor);
		AddItem(ActorPtr);
		ObjectHierarchy->ForEachChildRecursive(ActorPtr, [this, &AddItem](const TSoftObjectPtr<>&, const TSoftObjectPtr<>& ChildObject, EChildRelationship)
		{
			if (CanDisplayObject(ChildObject))
			{
				AddItem(ChildObject);
			}
			return EBreakBehavior::Continue;
		});
	}

	void SReplicationStreamViewer::GetObjectRowChildren(TSharedPtr<FReplicatedObjectData> ReplicatedObjectData, TFunctionRef<void(TSharedPtr<FReplicatedObjectData>)> ProcessChild)
	{
		// Important: this view should be possible to be built in programs, so it should not reference things like AActor, UActorComponent, ResolveObject, etc. directly.
		
		const TSoftObjectPtr<>& SearchedObject = ReplicatedObjectData->GetObjectPtr();
		if (!ObjectHierarchy)
		{
			return;
		}

		ObjectHierarchy->ForEachDirectChild(SearchedObject, [this, &ProcessChild](const TSoftObjectPtr<>& ChildObject, EChildRelationship Relationship)
		{
			if (const TSharedPtr<FReplicatedObjectData>* ObjectData = PathToObjectDataCache.Find(ChildObject.GetUniqueID()))
			{
				ProcessChild(*ObjectData);
			}
			return EBreakBehavior::Continue;
		});
	}
	bool SReplicationStreamViewer::CanDisplayObject(const TSoftObjectPtr<>& Object) const
	{
		const TOptional<IObjectHierarchyModel::FParentInfo> ParentInfo = ObjectHierarchy ? ObjectHierarchy->GetParentInfo(Object) : TOptional<IObjectHierarchyModel::FParentInfo>{};
		// If no hierarchy was provided during construction, only show actors. If hierarchy provided, check whether this type of subobject is allowed.
		const bool bCanShowWithinHierarchy = (ParentInfo && ShouldDisplayObjectRelation(ParentInfo->Relationship))
			|| ConcertSyncCore::IsActor(Object.GetUniqueID());
		const bool bDidDelegateAllow = !ShouldDisplayObjectDelegate.IsBound() || ShouldDisplayObjectDelegate.Execute(Object.GetUniqueID());
		return bCanShowWithinHierarchy && bDidDelegateAllow;
	}

	bool SReplicationStreamViewer::ShouldDisplayObjectRelation(EChildRelationship Relationship) const
	{
		const bool bSkipSubobject = Relationship == EChildRelationship::Subobject && !ObjectViewOptions.ShouldDisplaySubobjects();
		return !bSkipSubobject;
	}

	FSoftClassPath SReplicationStreamViewer::GetObjectClass(const TSoftObjectPtr<>& Object) const
	{
		const FSoftClassPath ResolvedClass = PropertiesModel->GetObjectClass(Object.GetUniqueID());
		if (ResolvedClass.IsValid())
		{
			return ResolvedClass;
		}

#if WITH_EDITOR
		// In the editor, we display the entire hierarchy (see BuildObjectHierarchyIfNeeded) so some items may not be in PropertiesModel.
		// Example: Add an actor with many components and assign nothing - all of those components will take this path.
		const UObject* LoadedObject = Object.Get();
		return LoadedObject ? LoadedObject->GetClass() : FSoftClassPath{};
#else
		// For non-editor, we should probably consider getting the class information through a delegate.
		return FSoftClassPath{};
#endif
	}
}

#undef LOCTEXT_NAMESPACE