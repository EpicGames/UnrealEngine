// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Editor/View/IReplicationStreamViewer.h"

#include "Replication/Editor/Model/Data/ReplicatedObjectData.h"
#include "Replication/Editor/View/Tree/SReplicationTreeView.h"
#include "Replication/Editor/View/Column/IObjectTreeColumn.h"
#include "Replication/Editor/View/Column/SelectionViewerColumns.h"
#include "SReplicatedPropertyView.h"
#include "StreamViewerObjectViewOptions.h"

#include "Algo/Transform.h"
#include "Misc/Optional.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SExpandableArea;
class SWidgetSwitcher;
struct FSoftClassPath;
struct FSoftObjectPath;

namespace UE::ConcertSharedSlate
{
	class SReplicatedPropertyView;
	class FPropertyData;
	class FReplicatedObjectData;
	class IEditableReplicationStreamModel;
	class IObjectHierarchyModel;
	class IObjectNameModel;
	class IPropertyAssignmentView;
	class IReplicationStreamModel;
	class SPropertyTreeView;
	
	enum class EChildRelationship : uint8;
	
	/**
	 * Root widget for viewing UMultiUserPropertyReplicationSelection.
	 * This widget knows how to display IReplicationStreamModel.
	 * 
	 * The underlying data is modified by SObjectToPropertyEditor, which uses this widget's extension
	 * points to call functions on IEditableReplicationStreamModel.
	 *
	 * Important: this view should be possible to be built in programs, so it should not reference things like AActor,
	 * UActorComponent, ResolveObject, etc. directly. 
	 */
	class SReplicationStreamViewer : public IReplicationStreamViewer
	{
	public:

		SLATE_BEGIN_ARGS(SReplicationStreamViewer)
		{}
			/** In the lower half of the editor, this view presents the properties associated with the object that is currently selected in the upper part of the view. */
			SLATE_ARGUMENT(TSharedPtr<IPropertyAssignmentView>, PropertyAssignmentView)
		
			/** Additional columns to add to the object view */
			SLATE_ARGUMENT(TArray<FObjectColumnEntry>, ObjectColumns)
		
			/** Initial primary sort to set. */
			SLATE_ARGUMENT(FColumnSortInfo, PrimaryObjectSort)
			/** Initial secondary sort to set. */
			SLATE_ARGUMENT(FColumnSortInfo, SecondaryObjectSort)

			/** Optional. If set, this determines the children nested under the root objects. */
			SLATE_ARGUMENT(TSharedPtr<IObjectHierarchyModel>, ObjectHierarchy)
			/** Optional. If set, this determines the display text for objects. */
			SLATE_ARGUMENT(TSharedPtr<IObjectNameModel>, NameModel)

			/** Optional. Called when the delete key is pressed in the object view. */
			SLATE_EVENT(SReplicationTreeView<FReplicatedObjectData>::FDeleteItems, OnDeleteObjects)
		
			/** Called to generate the context menu for objects. */
			SLATE_EVENT(FOnContextMenuOpening, OnObjectsContextMenuOpening)
			
		
			/** Optional. Whether a given object should be displayed. If this returns false on an object, none of its children will be shown either. */
			SLATE_EVENT(FShouldDisplayObject, ShouldDisplayObject)
		
			/** Optional. Gets the content to overlay on hovered rows; it covers the entire row. */
			SLATE_EVENT(SReplicationTreeView<FReplicatedObjectData>::FGetHoveredRowContent, GetHoveredRowContent)
			
			/** Optional widget to add to the left of the object list search bar. */
			SLATE_NAMED_SLOT(FArguments, LeftOfObjectSearchBar)
			/** Optional widget to add to the right of the object list search bar. */
			SLATE_NAMED_SLOT(FArguments, RightOfObjectSearchBar)

			/** Optional text to display when no object is in the outliner. Defaults to "No objects to display." "*/
			SLATE_ATTRIBUTE(FText, NoOutlinerObjects)

			/** Optional. Returns new widget that wraps the outliner widget, which displays the replicated objects. Could return e.g. an SOverlay or SDropTarget. */
			SLATE_EVENT(FWrapOutlinerWidget, WrapOutliner)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<IReplicationStreamModel>& InPropertiesModel);

		//~ Begin IReplicationStreamViewer Interface
		virtual void Refresh() override;
		virtual void RequestObjectColumnResort(const FName& ColumnId) override;
		virtual void RequestPropertyColumnResort(const FName& ColumnId) override;
		virtual void SetSelectedObjects(TConstArrayView<TSoftObjectPtr<>> Objects) override;
		virtual TArray<TSoftObjectPtr<>> GetSelectedObjects() const override;
		//~ End IReplicationStreamViewer Interface

		void RequestObjectDataRefresh() { bHasRequestedObjectRefresh = true; }
		void RequestPropertyDataRefresh() { bHasRequestedPropertyRefresh = true; }

		/** Selects the given objects. */
		void SelectObjects(TConstArrayView<TSoftObjectPtr<>> Objects, bool bAtEndOfTick = false);
		/** Expands the given objects, recursively if desired. */
		void ExpandObjects(TConstArrayView<TSoftObjectPtr<>> Objects, bool bRecursive, bool bAtEndOfTick = false);

		/** @return Whether Object is being displayed in the top object panel. */
		bool IsDisplayedInTopView(const FSoftObjectPath& Object) const;

		/** @return Gets the root objects selected in the outliner; the subobject view chooses which of these objects (or their subobjects) end up in GetSelectedObjectShowingProperties. */
		TArray<TSharedPtr<FReplicatedObjectData>> GetSelectedObjectItems() const;

		//~ Begin SWidget Interface
		virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
		//~ End SWidget Interface
		
	private:

		/** The model this view is visualizing. */
		TSharedPtr<IReplicationStreamModel> PropertiesModel;
		/** Can be null. If set, this determines the children nested under the root objects. Editor builds have access to e.g. to USubobjectDataSubsystem but programs do not. */
		TSharedPtr<IObjectHierarchyModel> ObjectHierarchy;
		/** Can be null. If set, this determines the display text for objects. Editor builds have access to e.g. to USubobjectDataSubsystem but programs do not. */
		TSharedPtr<IObjectNameModel> NameModel;

		/** Lists the properties of the selected actor */
		TSharedPtr<SExpandableArea> PropertyArea;
		/** Edits the property list and (optionally) exposes subobjects of the selected root object. */
		TSharedPtr<SReplicatedPropertyView> PropertySection;

		/** Tree view for replicated objects. */
		TSharedPtr<SReplicationTreeView<FReplicatedObjectData>> ReplicatedObjects;
		
		/** All object row data */
		TArray<TSharedPtr<FReplicatedObjectData>> AllObjectRowData;
		/** The instances of ObjectRowData which do not have any parents. This acts as the item source for the tree view. */
		TArray<TSharedPtr<FReplicatedObjectData>> RootObjectRowData;
		/** Inverse map of ObjectRowData using FReplicatedObjectData::GetObjectPath as key. Contains all elements of ObjectRowData. */
		TMap<FSoftObjectPath, TSharedPtr<FReplicatedObjectData>> PathToObjectDataCache;

		bool bIsPropertyAreaExpanded = false;
		/** View options for the object outliner. */
		FStreamViewerObjectViewOptions ObjectViewOptions;

		bool bHasRequestedObjectRefresh = false;
		bool bHasRequestedPropertyRefresh = false;
		TArray<TSoftObjectPtr<>> PendingToSelect;
		TArray<TSoftObjectPtr<>> PendingToExpand;
		bool bPendingExpandRecursively = false;
		
		/** Optional. Whether a given object should be displayed. If this returns false on an object, none of its children will be shown either. */
		FShouldDisplayObject ShouldDisplayObjectDelegate;

		// Widget creation helpers
		TSharedRef<SWidget> CreateContentWidget(const FArguments& InArgs);
		TSharedRef<SWidget> CreateOutlinerSection(const FArguments& InArgs);
		TSharedRef<SWidget> CreatePropertiesSection(const FArguments& InArgs);

		/** Regenerates object row data in the top view */
		void RefreshObjectData();
		/** Lists all objects that need an object in the top-view. */
		void IterateDisplayableObjects(TFunctionRef<void(const FSoftObjectPath& Object)> Delegate) const;
		
		/** Sets RootObjectRowData to all non-root nodes from ObjectRowData. */
		void BuildRootObjectRowData();

		/** Creates an item for every object in the hierarchy of ReplicatedObjectData */
		void BuildObjectHierarchyIfNeeded(TSharedPtr<FReplicatedObjectData> ReplicatedObjectData, TMap<FSoftObjectPath, TSharedPtr<FReplicatedObjectData>>& NewPathToObjectDataCache);
		void GetObjectRowChildren(TSharedPtr<FReplicatedObjectData> ReplicatedObjectData, TFunctionRef<void(TSharedPtr<FReplicatedObjectData>)> ProcessChild);
		
		/** Handles how much space the 'Clients' area uses with respect to its expansion state. */
		SSplitter::ESizeRule GetPropertyAreaSizeRule() const { return bIsPropertyAreaExpanded ? SSplitter::ESizeRule::FractionOfParent : SSplitter::ESizeRule::SizeToContent; }
		void OnPropertyAreaExpansionChanged(bool bExpanded) { bIsPropertyAreaExpanded = bExpanded; }

		/** Called in response to subobject display view option being changed. Rebuilds the entire hierarchy. */
		void OnSubobjectViewOptionToggled()
		{
			RequestObjectDataRefresh();
			RequestPropertyDataRefresh();
		}
		
		/** Invokes the ShouldDisplayObjectDelegate to determine whether ObjectPath should be displayed. */
		bool CanDisplayObject(const TSoftObjectPtr<>& Object) const;
		bool CanDisplayObject(const FSoftObjectPath& ObjectPath) const { return CanDisplayObject(TSoftObjectPtr<>(ObjectPath)); }
		/** Whether the view options allow this type of relationship to be shown. */
		bool ShouldDisplayObjectRelation(EChildRelationship Relationship) const;
		
		FSoftClassPath GetObjectClass(const TSoftObjectPtr<>& Object) const;
	};
}
