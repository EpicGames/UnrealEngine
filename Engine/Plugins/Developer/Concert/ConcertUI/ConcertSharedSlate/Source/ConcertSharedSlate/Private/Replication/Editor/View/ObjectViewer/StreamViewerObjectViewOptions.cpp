// Copyright Epic Games, Inc. All Rights Reserved.

#include "StreamViewerObjectViewOptions.h"

#include "ConcertFrontendUtils.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "FStreamViewerObjectViewOptions"

namespace UE::ConcertSharedSlate
{
	TSharedRef<SWidget> FStreamViewerObjectViewOptions::MakeViewOptionsComboButton()
	{
		return ConcertFrontendUtils::CreateViewOptionsComboButton(
			FOnGetContent::CreateRaw(this, &FStreamViewerObjectViewOptions::MakeMenuWidget)
			);
	}

	TSharedRef<SWidget> FStreamViewerObjectViewOptions::MakeMenuWidget()
	{
		FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("DisplaySubobjects.Label", "Display Subobjects"),
			LOCTEXT("DisplaySubobjects.Tooltip", "Whether to display subobjects (with the exception of components, which are continued to be displayed).\nSubobjects are objects nested under other objects."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this](){ bDisplaySubobjects = !bDisplaySubobjects; OnDisplaySubobjectsToggledDelegate.Broadcast(); }),
				FCanExecuteAction::CreateLambda([] { return true; }),
				FIsActionChecked::CreateLambda([this] { return bDisplaySubobjects; })),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		return MenuBuilder.MakeWidget();
	}
}

#undef LOCTEXT_NAMESPACE