// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"

class SWidget;

namespace UE::ConcertSharedSlate
{
	/** View options for the outliner displayed by SReplicationStreamViewer. */
	class FStreamViewerObjectViewOptions
    {
    public:

		/** Makes a standard View Options widget, displaying the eye ball icon and showing the possible options. */
		TSharedRef<SWidget> MakeViewOptionsComboButton();
		/** Returns a Menu widgets containing the available options. */
		TSharedRef<SWidget> MakeMenuWidget();

		/** Whether to display subobjects (with the exception of components, which are continued to be displayed). */
		bool ShouldDisplaySubobjects() const { return bDisplaySubobjects; }

		DECLARE_MULTICAST_DELEGATE(FOnDisplaySubobjectsToggled);
		/** Called when the option for displaying subobjects has been changed. */
		FOnDisplaySubobjectsToggled& OnDisplaySubobjectsToggled() { return OnDisplaySubobjectsToggledDelegate; }
		
	private:

		/** Called when the option for displaying subobjects has been changed. */
		FOnDisplaySubobjectsToggled OnDisplaySubobjectsToggledDelegate;
		
		/** Whether to display subobjects (with the exception of components, which are continued to be displayed). */
		bool bDisplaySubobjects = false;
    };
}

