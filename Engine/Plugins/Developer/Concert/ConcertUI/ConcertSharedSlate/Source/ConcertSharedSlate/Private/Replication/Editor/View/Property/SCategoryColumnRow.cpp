// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCategoryColumnRow.h"

#include "Delegates/DelegateCombinations.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"

namespace UE::ConcertSharedSlate
{
	void SCategoryColumnRow::Construct(const FArguments& InArgs, TSharedRef<STableViewBase> InOwner)
	{
		// rebuilds the whole table row from scratch
		ChildSlot
		.Padding(0.0f, 2.0f, 0.0f, 0.0f)
		[
			SNew(SBox)
			.MaxDesiredHeight(FConcertFrontendStyle::Get()->GetFloat("Concert.Replication.Tree.RowHeight"))
			[
				SNew(SBorder)
				.BorderImage(this, &SCategoryColumnRow::GetBackgroundImage)
				.Padding(FMargin(0.0f, 3.0f))
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(2.0f, 2.0f, 2.0f, 2.0f)
					.AutoWidth()
					[
						SNew(SExpanderArrow, SCategoryColumnRow::SharedThis(this))
					]

					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						InArgs._Content.Widget
					]
				]
			]
		];

		ConstructInternal(
			Super::FArguments()
				.Style(FAppStyle::Get(), "DetailsView.TreeView.TableRow")
				.ShowSelection(false),
			InOwner
		);
	}

	FReply SCategoryColumnRow::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			ToggleExpansion();
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	FReply SCategoryColumnRow::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
	{
		return OnMouseButtonDown(InMyGeometry, InMouseEvent);
	}

	const FSlateBrush* SCategoryColumnRow::GetBackgroundImage() const
	{
		if (IsHovered())
		{
			return IsItemExpanded()
				? FAppStyle::GetBrush("DetailsView.CategoryTop_Hovered")
				: FAppStyle::GetBrush("DetailsView.CollapsedCategory_Hovered");
		}
		else
		{
			return IsItemExpanded()
				? FAppStyle::GetBrush("DetailsView.CategoryTop")
				: FAppStyle::GetBrush("DetailsView.CollapsedCategory");
		}
	}
}
