// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertFrontendStyle.h"

#include "Delegates/DelegateCombinations.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Replication/Editor/View/Column/IReplicationTreeColumn.h"

namespace UE::ConcertSharedSlate
{
	class FPropertyNodeData;
}

namespace UE::ConcertSharedSlate
{
	/** A row that generates its columns generically via IReplicationTreeColumn. */
	class SCategoryColumnRow : public STableRow<TSharedPtr<FPropertyNodeData>>
	{
		using Super = STableRow;
	public:
		
		SLATE_BEGIN_ARGS(SCategoryColumnRow)
		{}
			SLATE_NAMED_SLOT(FArguments, Content)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedRef<STableViewBase> InOwner);
		
		virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

	private:

		TSharedPtr<SWidget> Content;
		
		const FSlateBrush* GetBackgroundImage() const;
	};
}
