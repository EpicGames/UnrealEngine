// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMultiObjectAssignment.h"

#include "Misc/EBreakBehavior.h"
#include "Replication/Data/ConcertPropertySelection.h"
#include "Replication/Editor/Model/Object/IObjectHierarchyModel.h"
#include "Replication/Editor/Utils/PropertyEnumerationUtils.h"

namespace UE::ConcertSharedSlate
{
	namespace Private
	{
		static TMap<FSoftObjectPath, TArray<FSoftObjectPath>> BuildHierarchy(const TArray<TSoftObjectPtr<>>& RootObjects, IObjectHierarchyModel& HierarchyModel, bool bShouldShowSubobjects)
		{
			TMap<FSoftObjectPath, TArray<FSoftObjectPath>> Result;

			for (const TSoftObjectPtr<>& Object : RootObjects)
			{
				HierarchyModel.ForEachChildRecursive(
					Object,
					[&Result](const TSoftObjectPtr<>& Parent, const TSoftObjectPtr<>& ChildObject, EChildRelationship Relationship)
					{
						// ForEachChildRecursive iterates through the hierarchy as it would be displayed by the component panel in the details panel
						// E.g. for ACharacter this would say
						//		Parent		= /Game/Maps.Map:PersistentLevel.Character.CapsuleComponent
						//		ChildObject = /Game/Maps.Map:PersistentLevel.Character.CharacterMesh0
						// However, we want Result to contain { /Game/Maps.Map:PersistentLevel.Character, /Game/Maps.Map:PersistentLevel.Character.CharacterMesh0 }
						
						// E.g. "/Game/Maps.Map:PersistentLevel.Actor.Component"
						FString ParentPathString = ChildObject.ToString();
						int32 Index = INDEX_NONE;
						ParentPathString.FindLastChar('.', Index);
						ParentPathString.LeftInline(Index);

						const FSoftObjectPath ParentPath(*ParentPathString);
						Result.FindOrAdd(ParentPath).Add(ChildObject.GetUniqueID());
						return EBreakBehavior::Continue;
					},
					bShouldShowSubobjects ? EChildRelationshipFlags::All : EChildRelationshipFlags::Component
					);
			}

			return Result;
		}

		static void ForEachCategory(
			const TArray<TSoftObjectPtr<>>& Start,
			const TMap<FSoftObjectPath, TArray<FSoftObjectPath>>& Hierarchy,
			TFunctionRef<void(const TArray<TSoftObjectPtr<>>& ContextObjects)> Callback
			)
		{
			if (Start.IsEmpty())
			{
				return;
			}

			// While multi-editing, if any of the subobjects do not match skip the entire category.
			const TSoftObjectPtr<> FirstEntry = Start[0];
			const TArray<FSoftObjectPath>* FirstEntryChildren = Hierarchy.Find(FirstEntry.GetUniqueID());
			if (!FirstEntryChildren)
			{
				return;
			}

			// E.g. "/Game/Maps.Map:PersistentLevel.Actor"
			const FString Base = FirstEntry.ToString();
			for (const FSoftObjectPath& ChildPath : *FirstEntryChildren)
			{
				// E.g. "/Game/Maps.Map:PersistentLevel.Actor.Component"
				FString ChildName = ChildPath.ToString();
				// +1 for the "." preceding the child, otherwise we'd get ".Component"
				ChildName.RightChopInline(Base.Len() + 1);

				TArray<TSoftObjectPtr<>> RelatedChildren;
				// This intentionally iterates FirstEntry as well
				for (const TSoftObjectPtr<>& StartObject : Start)
				{
					const TArray<FSoftObjectPath>* Children = Hierarchy.Find(StartObject.GetUniqueID());
					if (!Children)
					{
						continue;
					}

					const FString ExpectedChildPathString = Base + TEXT(".") + ChildName;
					const FSoftObjectPath ExpectedChildPath(*ExpectedChildPathString);
					if (Children->Contains(ExpectedChildPath))
					{
						RelatedChildren.Emplace(ExpectedChildPath);
					}
				}

				if (!RelatedChildren.IsEmpty())
				{
					Callback(RelatedChildren);
					ForEachCategory(RelatedChildren, Hierarchy, Callback);
				}
			}
		}
	}
	
	void SMultiObjectAssignment::Construct(const FArguments& InArgs, TSharedRef<IPropertyTreeView> InTreeView)
	{
		TreeView = InTreeView;
		ObjectHierarchy = InArgs._ObjectHierarchy;
		OptionalPropertySource = InArgs._PropertySource;
		
		ChildSlot
		[
			TreeView->GetWidget()
		];
	}

	void SMultiObjectAssignment::RefreshData(const TArray<TSoftObjectPtr<>>& Objects, const IReplicationStreamModel& Model)
	{
		TArray<FPropertyAssignmentEntry> Entries;
		// The root objects are always "displayed" for the purposes of GetDisplayedGroups
		DisplayedGroups = { FObjectGroup{ Objects } };
		
		auto[RootEntry, bRootSharesClass] = BuildAssignmentEntry(Objects, Model);
		if (!RootEntry.PropertiesToDisplay.IsEmpty() && ensureMsgf(bRootSharesClass, TEXT("Objects do not share the same class. Investigate invalid call.")))
		{
			Entries.Emplace(MoveTemp(RootEntry));
		}

		if (ObjectHierarchy)
		{
			using namespace Private;
			const TMap<FSoftObjectPath, TArray<FSoftObjectPath>> HierarchyInfo = BuildHierarchy(Objects, *ObjectHierarchy, bShouldShowSubobjects);
			ForEachCategory(Objects, HierarchyInfo, [this, &Model, &Entries](const TArray<TSoftObjectPtr<>>& ContextObjects)
			{
				DisplayedGroups.Add(FObjectGroup{ ContextObjects });
				
				auto[Entry, bShareClass] = BuildAssignmentEntry(ContextObjects, Model);
				if (bShareClass)
				{
					Entries.Emplace(MoveTemp(Entry));
				}
			});
		}

		if (!Entries.IsEmpty())
		{
			const bool bObjectsHaveChanged = PreviousSelectedObjects != Objects;
			PreviousSelectedObjects = Objects;
		
			// If the objects have changed, the classes may share properties.
			// In that case, below we'd reuse the item pointer, which would cause the tree view to re-use the old row widgets.
			// However, we must regenerate all column widgets since they may be referencing the object the row was originally built for. So they'd display the state of the previous object still!
			// Example: Assign property combo-box in Multi-User All Clients view displays who has the property assigned.
			// Note: If the objects did not change, we definitely want to reuse item pointers since otherwise the user row selection is reset.
			const bool bCanReusePropertyData = !bObjectsHaveChanged;
		
			TreeView->RefreshPropertyData(Entries, bCanReusePropertyData);

			if (bObjectsHaveChanged)
			{
				OnObjectGroupsChangedDelegate.Broadcast();
			}
		}
		else
		{
			PreviousSelectedObjects.Empty();
			TreeView->RefreshPropertyData({}, false);
			OnObjectGroupsChangedDelegate.Broadcast();
		}
	}

	void SMultiObjectAssignment::SetShouldShowSubobjects(bool bShowSubobjects)
	{
		if (bShouldShowSubobjects != bShowSubobjects)
		{
			bShouldShowSubobjects = bShowSubobjects;
			// TODO UE-216097: Refresh UI
		}
	}

	SMultiObjectAssignment::FBuildAssignmentEntryResult SMultiObjectAssignment::BuildAssignmentEntry(const TArray<TSoftObjectPtr<>>& Objects, const IReplicationStreamModel& Model)
	{
		FBuildAssignmentEntryResult Result;

		Result.Entry = { .ContextObjects = Objects };
		TSet<FConcertPropertyChain>& Properties = Result.Entry.PropertiesToDisplay;
		FSoftClassPath& ClassPath = Result.Entry.Class;
		
		bool& bHasClassPath = Result.bHaveSharedClass;
		EnumerateProperties(Objects, Model, OptionalPropertySource.Get(),
			[&Properties, &bHasClassPath, &ClassPath](const FSoftClassPath& Class, const FConcertPropertyChain& Chain)
			{
				if (!bHasClassPath)
				{
					bHasClassPath = true;
					ClassPath = Class;
				}
				else if (ClassPath != Class)
				{
					// Duplicate class - caller of RefreshData should not have called us but handle it gracefully.
					bHasClassPath = false;
					return EBreakBehavior::Break;
				}
				
				Properties.Add(Chain);
				return EBreakBehavior::Continue;
			});

		return Result;
	}
}
