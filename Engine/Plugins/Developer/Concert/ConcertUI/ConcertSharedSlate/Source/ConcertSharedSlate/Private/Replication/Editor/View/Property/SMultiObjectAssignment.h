// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Editor/View/IMultiObjectPropertyAssignmentView.h"
#include "Replication/Editor/View/IPropertyAssignmentView.h"
#include "Replication/Editor/View/IPropertyTreeView.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

template<class T>
struct TSoftObjectPtr;

namespace UE::ConcertSharedSlate
{
	class IObjectHierarchyModel;
	class IPropertySourceProcessor;
	class IReplicationStreamModel;
	
	/** SMultiObjectAssignment shows the properties of the displayed object and all of its subobjects.*/
	class SMultiObjectAssignment
		: public SCompoundWidget
		, public IMultiObjectPropertyAssignmentView
	{
	public:

		SLATE_BEGIN_ARGS(SMultiObjectAssignment){}
			/** Optional. If specified, displays the properties of this model instead of those assigned in the stream. */
			SLATE_ARGUMENT(TSharedPtr<IPropertySourceProcessor>, PropertySource)
			
			/** Optional. Gets components and subobjects of the displayed object. If unspecified, behaves exactly like SPerObjectAssignmentView. */
			SLATE_ARGUMENT(TSharedPtr<IObjectHierarchyModel>, ObjectHierarchy)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedRef<IPropertyTreeView> InTreeView);

		//~ Begin IPropertyAssignmentView Interface
		virtual void RefreshData(const TArray<TSoftObjectPtr<>>& Objects, const IReplicationStreamModel& Model) override;
		virtual void RequestRefilter() const override { return TreeView->RequestRefilter(); }
		virtual void RequestResortForColumn(const FName& ColumnId) override { return TreeView->RequestRefilter(); }
		virtual TSharedRef<SWidget> GetWidget() override { return SharedThis(this); }
		virtual TArray<FObjectGroup> GetDisplayedGroups() const override { return DisplayedGroups; }
		virtual FOnSelectionChanged& OnObjectGroupsChanged() override { return OnObjectGroupsChangedDelegate; }
		//~ End IPropertyAssignmentView Interface
		
		//~ End IMultiObjectPropertyAssignmentView Interface
		virtual void SetShouldShowSubobjects(bool bShowSubobjects) override;
		virtual bool GetShouldShowSubobjects() const override { return bShouldShowSubobjects;}
		//~ End IMultiObjectPropertyAssignmentView Interface

	private:

		/** The tree view that is being wrapped. */
		TSharedPtr<IPropertyTreeView> TreeView;
		/** Used to determine whether to rebuild the entire property data. */
		TArray<TSoftObjectPtr<>> PreviousSelectedObjects;
		/** Cached value for GetDisplayedGroups. */
		TArray<FObjectGroup> DisplayedGroups;

		/** Used to get subobjects of selected objects. */
		TSharedPtr<IObjectHierarchyModel> ObjectHierarchy;
		/** Optional. If specified, displays the properties of this model instead of those assigned in the stream. */
		TSharedPtr<IPropertySourceProcessor> OptionalPropertySource;

		/** Whether EChildRelationshipFlags::Subobject objects should be shown. */
		bool bShouldShowSubobjects = false;
		
		/** Broadcasts when the result of GetDisplayedGroups() has changed. */
		FOnSelectionChanged OnObjectGroupsChangedDelegate;

		struct FBuildAssignmentEntryResult
		{
			FPropertyAssignmentEntry Entry;
			bool bHaveSharedClass = false;
		};
		/** Builds a property section grouped by Objects (usually has 1 object - contains similar objects, like StaticMeshComponent0, for multi-edit purposes). */
		FBuildAssignmentEntryResult BuildAssignmentEntry(const TArray<TSoftObjectPtr<>>& Objects, const IReplicationStreamModel& Model);
	};
}
