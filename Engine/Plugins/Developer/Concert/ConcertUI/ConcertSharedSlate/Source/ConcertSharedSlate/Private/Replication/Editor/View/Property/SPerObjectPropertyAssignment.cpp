// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPerObjectPropertyAssignment.h"

#include "Misc/EBreakBehavior.h"
#include "Replication/Data/ConcertPropertySelection.h"
#include "Replication/Editor/Utils/PropertyEnumerationUtils.h"

namespace UE::ConcertSharedSlate
{
	void SPerObjectPropertyAssignment::Construct(const FArguments& InArgs, TSharedRef<IPropertyTreeView> InTreeView)
	{
		TreeView = MoveTemp(InTreeView);
		OptionalPropertySource = InArgs._PropertySource;
		
		ChildSlot
		[
			TreeView->GetWidget()
		];
	}

	void SPerObjectPropertyAssignment::RefreshData(const TArray<TSoftObjectPtr<>>& Objects, const IReplicationStreamModel& Model)
	{
		DisplayedGroups = { FObjectGroup{ Objects } };
		
		FPropertyAssignmentEntry AssignmentEntry { .ContextObjects = Objects };
		TSet<FConcertPropertyChain>& Properties = AssignmentEntry.PropertiesToDisplay;
		FSoftClassPath& ClassPath = AssignmentEntry.Class;
		
		bool bHasClassPath = false;
		EnumerateProperties(Objects, Model, OptionalPropertySource.Get(),
			[&Properties, &bHasClassPath, &ClassPath](const FSoftClassPath& Class, const FConcertPropertyChain& Chain)
			{
				if (!bHasClassPath)
				{
					bHasClassPath = true;
					ClassPath = Class;
				}
				else if (ClassPath != Class)
				{
					// Duplicate class - caller of RefreshData should not have called us but handle it gracefully.
					bHasClassPath = false;
					return EBreakBehavior::Break;
				}
				
				Properties.Add(Chain);
				return EBreakBehavior::Continue;
			});
		
		if (bHasClassPath && !Properties.IsEmpty())
		{
			const bool bObjectsHaveChanged = PreviousSelectedObjects != Objects;
			PreviousSelectedObjects = Objects;
			
			// If the objects have changed, the classes may share properties.
			// In that case, below we'd reuse the item pointer, which would cause the tree view to re-use the old row widgets.
			// However, we must regenerate all column widgets since they may be referencing the object the row was originally built for. So they'd display the state of the previous object still!
			// Example: Assign property combo-box in Multi-User All Clients view displays who has the property assigned.
			// Note: If the objects did not change, we definitely want to reuse item pointers since otherwise the user row selection is reset.
			const bool bCanReusePropertyData = !bObjectsHaveChanged;
			TreeView->RefreshPropertyData({ AssignmentEntry }, bCanReusePropertyData);

			if (!bObjectsHaveChanged)
			{
				OnObjectGroupsChangedDelegate.Broadcast();
			}
		}
		else
		{
			PreviousSelectedObjects.Empty();
			TreeView->RefreshPropertyData({}, false);
			OnObjectGroupsChangedDelegate.Broadcast();
		}
	}
}
