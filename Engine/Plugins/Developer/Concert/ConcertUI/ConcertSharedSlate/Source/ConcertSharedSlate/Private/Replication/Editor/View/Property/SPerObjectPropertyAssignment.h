// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Editor/View/IPropertyAssignmentView.h"
#include "Replication/Editor/View/IPropertyTreeView.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::ConcertSharedSlate
{
	class IPropertySourceProcessor;
	class IReplicationStreamModel;
	
	/** SPerObjectPropertyAssignment shows only the properties of the displayed object, which is achieved by wrapping SPropertyTreeView. */
	class SPerObjectPropertyAssignment
		: public SCompoundWidget
		, public IPropertyAssignmentView
	{
	public:

		SLATE_BEGIN_ARGS(SPerObjectPropertyAssignment){}
			/** Optional. If specified, displays the properties of this model instead of those assigned in the stream. */
			SLATE_ARGUMENT(TSharedPtr<IPropertySourceProcessor>, PropertySource)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedRef<IPropertyTreeView> InTreeView);

		//~ Begin IPropertyAssignmentView Interface
		virtual void RefreshData(const TArray<TSoftObjectPtr<>>& Objects, const IReplicationStreamModel& Model) override;
		virtual void RequestRefilter() const override { return TreeView->RequestRefilter(); }
		virtual void RequestResortForColumn(const FName& ColumnId) override { return TreeView->RequestRefilter(); }
		virtual TSharedRef<SWidget> GetWidget() override { return SharedThis(this); }
		virtual TArray<FObjectGroup> GetDisplayedGroups() const override { return DisplayedGroups; }
		virtual FOnSelectionChanged& OnObjectGroupsChanged() override { return OnObjectGroupsChangedDelegate; }
		//~ End IPropertyAssignmentView Interface

	private:

		/** The tree view that is being wrapped. */
		TSharedPtr<IPropertyTreeView> TreeView;
		/** Used to determine whether to rebuild the entire property data. */
		TArray<TSoftObjectPtr<>> PreviousSelectedObjects;
		/** Cached value for GetDisplayedGroups. */
		TArray<FObjectGroup> DisplayedGroups;
		
		/** Optional. If specified, displays the properties of this model instead of those assigned in the stream. */
		TSharedPtr<IPropertySourceProcessor> OptionalPropertySource;
		
		/** Broadcasts when the result of GetDisplayedGroups() has changed. */
		FOnSelectionChanged OnObjectGroupsChangedDelegate;
	};
}


