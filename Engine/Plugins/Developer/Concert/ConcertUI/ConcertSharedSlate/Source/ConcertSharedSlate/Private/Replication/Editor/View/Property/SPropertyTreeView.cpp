// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPropertyTreeView.h"

#include "Replication/Editor/Model/Data/PropertyData.h"
#include "Replication/Editor/Model/Data/PropertyNodeData.h"
#include "Replication/Editor/View/Column/PropertyColumnAdapter.h"
#include "SCategoryColumnRow.h"
#include "Trace/ConcertTrace.h"

#include "Algo/Transform.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "SReplicatedPropertiesView"

namespace UE::ConcertSharedSlate
{
	namespace Private
	{
		static TSharedRef<FPropertyNodeData> AllocateNode_Property(TArray<TSoftObjectPtr<>> ContextObjects, FSoftClassPath OwningClass, FConcertPropertyChain PropertyChain)
		{
			return MakeShared<FPropertyNodeData>(FPropertyData{ MoveTemp(ContextObjects), MoveTemp(OwningClass), MoveTemp(PropertyChain) });
		}

		static TSharedRef<FPropertyNodeData> AllocateNode_Category(TArray<TSoftObjectPtr<>> ContextObjects)
		{
			return MakeShared<FPropertyNodeData>(FCategoryData{ ContextObjects });
		}

		static const TArray<TSoftObjectPtr<>>& GetContext(const FPropertyNodeData& ode)
		{
			return ode.IsCategoryNode()
				? ode.GetCategoryData()->GetContextObjects()
				: ode.GetPropertyData()->GetContextObjects();
		}
	}
	
	void SPropertyTreeView::Construct(const FArguments& InArgs)
	{
		FilterDelegate = InArgs._FilterItem;
		CreateCategoryRowDelegate = InArgs._CreateCategoryRow;
		
		ChildSlot
		[
			SAssignNew(TreeView, SReplicationTreeView<FPropertyNodeData>)
				.RootItemsSource(&RootPropertyNodes)
				.OnGetChildren(this, &SPropertyTreeView::GetPropertyRowChildren)
				.FilterItem(this, &SPropertyTreeView::FilterItem)
				.OverrideRowWidget(this, &SPropertyTreeView::OverrideRowWidget)
				.OverrideIsLessThan(this, &SPropertyTreeView::OverrideIsLessThan)
				.OverrideGetSearchTerms(this, &SPropertyTreeView::OverrideGetSearchTerms)
				.Columns(FPropertyColumnAdapter::Transform(InArgs._Columns))
				.ExpandableColumnLabel(InArgs._ExpandableColumnLabel)
				.PrimarySort(InArgs._PrimarySort)
				.SecondarySort(InArgs._SecondarySort)
				.SelectionMode(InArgs._SelectionMode)
				.LeftOfSearchBar() [ InArgs._LeftOfSearchBar.Widget ]
				.RightOfSearchBar() [ InArgs._RightOfSearchBar.Widget ]
				.RowBelowSearchBar() [ InArgs._RowBelowSearchBar.Widget ]
				.NoItemsContent() [ InArgs._NoItemsContent.Widget ]
		];
	}

	void SPropertyTreeView::RefreshPropertyData(const TArray<FPropertyAssignmentEntry>& Entries, bool bCanReuseExistingRowItems)
	{
		SCOPED_CONCERT_TRACE(RefreshPropertyData);
		if (!bCanReuseExistingRowItems)
		{
			ChainToPropertyDataCache.Reset();
			CategoryNodes.Reset();
		}
		
		PropertyRowData.Empty();
		{
			// Try to re-use old instances. This is also done so the expansion states restore correctly in the tree view.
			TMap<FConcertPropertyChain, TArray<TSharedPtr<FPropertyNodeData>>> NewChainToPropertyDataCache;
			TMap<TSoftObjectPtr<>, FCategoryMetaData> NewCategoryNodes;

			if (ShouldBuildCategories())
			{
				RefreshCategoryNodes(Entries, NewCategoryNodes);
			}
			RefreshPropertyNodes(Entries, NewChainToPropertyDataCache);

			// If an item was removed, then NewPathToPropertyDataCache does not contain it. 
			ChainToPropertyDataCache = MoveTemp(NewChainToPropertyDataCache);
			CategoryNodes = MoveTemp(NewCategoryNodes);
		}
		
		// The tree view requires the item source to only contain the root items.
		BuildRootPropertyRowData();
		TreeView->RequestRefilter();

		// Every time the tree is filled with a new hierarchy, all categories should be expanded
		if (!bCanReuseExistingRowItems && ShouldBuildCategories())
		{
			TArray<TSharedPtr<FPropertyNodeData>> Categories;
			Algo::Transform(CategoryNodes, Categories, [](const TPair<TSoftObjectPtr<>, FCategoryMetaData>& Pair){ return Pair.Value.Node.ToSharedPtr(); });
			TreeView->SetExpandedItems(Categories, true);
		}
	}

	void SPropertyTreeView::RequestScrollIntoView(const FConcertPropertyChain& PropertyChain)
	{
		const int32 Index = PropertyRowData.IndexOfByPredicate([&PropertyChain](const TSharedPtr<FPropertyNodeData>& Data)
		{
			const TOptional<FPropertyData>& PropertyData = Data->GetPropertyData();
			return PropertyData && PropertyData->GetProperty() == PropertyChain;
		});
		if (PropertyRowData.IsValidIndex(Index))
		{
			TreeView->SetExpandedItems({ PropertyRowData[Index] }, true);
			TreeView->RequestScrollIntoView(PropertyRowData[Index]);
		}
	}

	void SPropertyTreeView::RefreshCategoryNodes(const TArray<FPropertyAssignmentEntry>& Entries, TMap<TSoftObjectPtr<>, FCategoryMetaData>& NewCategoryNodes)
	{
		SourceEntriesForSorting.Empty();
				
		for (const FPropertyAssignmentEntry& Entry : Entries)
		{
			if (!ensure(!Entry.ContextObjects.IsEmpty()))
			{
				continue;
			}
			// Entry's category widget is supposed to appear in the same order as it was passed to us
			SourceEntriesForSorting.Add(Entry.ContextObjects);
			const TSoftObjectPtr<> FirstContextObject = Entry.ContextObjects[0];

			// Re-use existing FPropertyNodeData if possible (to retain selection in tree view)
			const FCategoryMetaData* ExistingItem = CategoryNodes.Find(FirstContextObject);
			if (!ExistingItem)
			{
				const TSharedRef<FPropertyNodeData> NodeData = Private::AllocateNode_Category(Entry.ContextObjects);
				const FCategoryRowGenerationArgs CategoryArgs(Entry.ContextObjects, TAttribute<FText>::CreateLambda([this](){ return TreeView->GetHighlightText(); }));
				const TSharedRef<ICategoryRow> CategoryRow = CreateCategoryRowDelegate.Execute(CategoryArgs);
				
				FCategoryMetaData& CategoryMetaData = CategoryNodes.Emplace(FirstContextObject, FCategoryMetaData(NodeData, CategoryRow));
				ExistingItem = &CategoryMetaData;
			}
			
			PropertyRowData.Emplace(ExistingItem->Node);

			// Even when multi-editing, only the first context object is mapped to the category.
			NewCategoryNodes.Emplace(FirstContextObject, *ExistingItem);
		}
	}

	void SPropertyTreeView::RefreshPropertyNodes(const TArray<FPropertyAssignmentEntry>& Entries, TMap<FConcertPropertyChain, TArray<TSharedPtr<FPropertyNodeData>>>& NewChainToPropertyDataCache)
	{
		for (const FPropertyAssignmentEntry& Entry : Entries)
		{
#if UE_BUILD_DEBUG
			TSet<FConcertPropertyChain> DuplicateChainDetection;
#endif
			
			for (const FConcertPropertyChain& PropertyChain : Entry.PropertiesToDisplay)
			{
#if UE_BUILD_DEBUG
				check(!DuplicateChainDetection.Contains(PropertyChain));
				DuplicateChainDetection.Add(PropertyChain);
#endif
				
				TSharedPtr<FPropertyNodeData> PreExisting = FindPropertyNode(PropertyChain, Entry.ContextObjects);
				const TSharedRef<FPropertyNodeData> Item = PreExisting
					? PreExisting.ToSharedRef()
					: Private::AllocateNode_Property(Entry.ContextObjects, Entry.Class, PropertyChain);
				PropertyRowData.Emplace(Item);
				NewChainToPropertyDataCache.FindOrAdd(PropertyChain).AddUnique(Item);
			}
		}
	}

	TSharedPtr<FPropertyNodeData> SPropertyTreeView::FindPropertyNode(const FConcertPropertyChain& PropertyChain, TConstArrayView<TSoftObjectPtr<>> ContextObjects)
	{
		const TArray<TSharedPtr<FPropertyNodeData>>* ExistingItems = ChainToPropertyDataCache.Find(PropertyChain);
		if (!ExistingItems)
		{
			return nullptr;
		}

		for (const TSharedPtr<FPropertyNodeData>& Item : *ExistingItems)
		{
			const TOptional<FPropertyData>& PropertyData = Item->GetPropertyData();
			const bool bIsProperty = PropertyData->GetProperty() == PropertyChain;
			const bool bReferencesContextObject = PropertyData->GetContextObjects().ContainsByPredicate([&ContextObjects](const TSoftObjectPtr<>& ContextObject)
			{
				return ContextObjects.Contains(ContextObject);
			});
			if (bIsProperty && bReferencesContextObject)
			{
				return Item;
			}
		}

		return nullptr;
	}

	void SPropertyTreeView::BuildRootPropertyRowData()
	{
		RootPropertyNodes.Empty(PropertyRowData.Num());

		if (ShouldBuildCategories())
		{
			Algo::TransformIf(PropertyRowData, RootPropertyNodes,
				[](const TSharedPtr<FPropertyNodeData>& NodeData){ return NodeData->IsCategoryNode(); },
				[](const TSharedPtr<FPropertyNodeData>& NodeData){ return NodeData; }
				);
		}
		else
		{
			Algo::TransformIf(PropertyRowData, RootPropertyNodes,
				[](const TSharedPtr<FPropertyNodeData>& NodeData)
				{
					const TOptional<FPropertyData>& PropertyData = NodeData->GetPropertyData();
					return PropertyData && PropertyData->GetProperty().IsRootProperty();
				},
				[](const TSharedPtr<FPropertyNodeData>& NodeData){ return NodeData; }
				);
		}
	}
	
	void SPropertyTreeView::GetPropertyRowChildren(TSharedPtr<FPropertyNodeData> ParentNodeData, TFunctionRef<void(TSharedPtr<FPropertyNodeData>)> ProcessChild) const
	{
		SCOPED_CONCERT_TRACE(GetPropertyRowChildren);
		
		if (ParentNodeData->IsCategoryNode())
		{
			const TOptional<FCategoryData>& ParentCategoryData = ParentNodeData->GetCategoryData();
			check(ParentCategoryData);
			EnumerateRootProperties(*ParentCategoryData, ProcessChild);
		}
		else
		{
			EnumerateChildProperties(*ParentNodeData, ProcessChild);
		}
	}
	
	void SPropertyTreeView::EnumerateRootProperties(const FCategoryData& ParentCategoryData, TFunctionRef<void(TSharedPtr<FPropertyNodeData>)> ProcessChild) const
	{
		for (const TSharedPtr<FPropertyNodeData>& NodeData : PropertyRowData)
		{
			const TOptional<FPropertyData>& PropertyData = NodeData->GetPropertyData();
			if (!PropertyData)
			{
				continue;
			}

			const TArray<TSoftObjectPtr<>>& ParentContextObjects = ParentCategoryData.GetContextObjects();
			const TArray<TSoftObjectPtr<>>& ContextObjects = PropertyData->GetContextObjects();
			const bool bHaveSameContextObject = ContextObjects == ParentContextObjects;
			if (bHaveSameContextObject && PropertyData->GetProperty().IsRootProperty())
			{
				ProcessChild(NodeData);
			}
		}
	}
	
	void SPropertyTreeView::EnumerateChildProperties(const FPropertyNodeData& ParentNode, TFunctionRef<void(TSharedPtr<FPropertyNodeData>)> ProcessChild) const
	{
		const TArray<TSoftObjectPtr<>>& ParentContext = Private::GetContext(ParentNode);
		const TOptional<FPropertyData>& ParentPropertyData = ParentNode.GetPropertyData();
		check(ParentPropertyData);
		
		// Not the most efficient but it should be fine.
		for (const TSharedPtr<FPropertyNodeData>& NodeData : PropertyRowData)
		{
			// There can be multiple components with the same property paths - exclude properties that are under different category nodes.
			const TArray<TSoftObjectPtr<>>& OtherNodeContext = Private::GetContext(*NodeData);
			const bool bContextIsSame = ParentContext == OtherNodeContext;
			if (!bContextIsSame)
			{
				continue;
			}
			
			const TOptional<FPropertyData>& PropertyData = NodeData->GetPropertyData();
			if (bContextIsSame && PropertyData && PropertyData->GetProperty().IsDirectChildOf(ParentPropertyData->GetProperty()))
			{
				ProcessChild(NodeData);
			}
		}
	}

	EItemFilterResult SPropertyTreeView::FilterItem(const FPropertyNodeData& PropertyNodeData) const
	{
		return PropertyNodeData.IsCategoryNode()
			? EItemFilterResult::IncludeOnlyIfChildIsIncluded
			: FilterDelegate.IsBound() ? FilterDelegate.Execute(*PropertyNodeData.GetPropertyData()) : EItemFilterResult::Include;
	}

	TSharedPtr<ITableRow> SPropertyTreeView::OverrideRowWidget(TSharedPtr<FPropertyNodeData> NodeData, const TSharedRef<STableViewBase>& TableViewBase, const TReplicationTreeData<FPropertyNodeData>::FGenerateRowArgs& Args)
	{
		if (NodeData->IsCategoryNode() && ensure(CreateCategoryRowDelegate.IsBound()))
		{
			const FCategoryMetaData* CategoryMetaData = FindCategoryMetaData(*NodeData);
			return SNew(SCategoryColumnRow, TableViewBase)
				.Content()
				[
					CategoryMetaData->RowWidgetContent->GetWidget()
				];
		}
		return nullptr;
	}

	EComparisonOverride SPropertyTreeView::OverrideIsLessThan(const TSharedPtr<FPropertyNodeData>& Left, const TSharedPtr<FPropertyNodeData>& Right) const
	{
		if (!Left->IsCategoryNode() && !Right->IsCategoryNode())
		{
			return EComparisonOverride::UseDefault;
		}
		
		const TArray<TSoftObjectPtr<>>& LeftContext = Private::GetContext(*Left);
		const TArray<TSoftObjectPtr<>>& RightContext = Private::GetContext(*Right);
		const int32 LeftIndex = SourceEntriesForSorting.IndexOfByKey(LeftContext);
		const int32 RightIndex = SourceEntriesForSorting.IndexOfByKey(RightContext);
		ensure(SourceEntriesForSorting.IsValidIndex(LeftIndex) && SourceEntriesForSorting.IsValidIndex(RightIndex));
		return LeftIndex < RightIndex ? EComparisonOverride::Less : EComparisonOverride::NotLess;
	}

	ESearchTermResult SPropertyTreeView::OverrideGetSearchTerms(const TSharedPtr<FPropertyNodeData>& NodeData, TArray<FString>& InOutSearchStrings) const
	{
		const FCategoryMetaData* CategoryMetaData = NodeData->IsCategoryNode() ? FindCategoryMetaData(*NodeData) : nullptr;
		if (CategoryMetaData)
		{
			CategoryMetaData->RowWidgetContent->GenerateSearchTerms(NodeData->GetCategoryData()->GetContextObjects(), InOutSearchStrings);
			return ESearchTermResult::UseOverrideOnly;
		}
		return ESearchTermResult::UseDefault;
	}
}

#undef LOCTEXT_NAMESPACE