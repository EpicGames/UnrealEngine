// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/PropertyTreeFactory.h"
#include "Replication/Data/ConcertPropertySelection.h"
#include "Replication/Editor/Model/Data/PropertyNodeData.h"
#include "Replication/Editor/View/IPropertyTreeView.h"
#include "Replication/Editor/View/Tree/SReplicationTreeView.h"

#include "Replication/Editor/View/Column/IPropertyTreeColumn.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::ConcertSharedSlate
{
	class ICategoryRow;
}

namespace UE::ConcertSharedSlate
{
	class FCategoryData;
	class FPropertyNodeData;
	
	/**
	 * This widget knows how to display a list of properties in a tree view.
	 * It generates the items and exposes extension points for more advanced UI, such as filtering.
	 */
	class SPropertyTreeView
		: public SCompoundWidget
		, public IPropertyTreeView
	{
	public:
		
		SLATE_BEGIN_ARGS(SPropertyTreeView)
		{}
			/*************** Arguments inherited by SReplicationTreeView ***************/
		
			/** Optional callback to do more filtering of items on top of the search bar. */
			SLATE_EVENT(SReplicationTreeView<FPropertyData>::FCustomFilter, FilterItem)
			
			/** Optional callback for creating category rows. If unset, no category rows are generated. */
			SLATE_EVENT(FCreateCategoryRow, CreateCategoryRow)
		
			/** The columns this list should have */
			SLATE_ARGUMENT(TArray<FPropertyColumnEntry>, Columns)
			/** The name of the column that will have the SExpanderArrow for the tree view. */
			SLATE_ARGUMENT(FName, ExpandableColumnLabel)
			/** Initial primary sort to set. */
			SLATE_ARGUMENT(FColumnSortInfo, PrimarySort)
			/** Initial secondary sort to set. */
			SLATE_ARGUMENT(FColumnSortInfo, SecondarySort)
		
			/** How many items are to allowed to be selected */
			SLATE_ARGUMENT(ESelectionMode::Type, SelectionMode)
		
			/** Optional widget to add to the left of the search bar. */
			SLATE_NAMED_SLOT(FArguments, LeftOfSearchBar)
			/** Optional widget to add to the left of the search bar. */
			SLATE_NAMED_SLOT(FArguments, RightOfSearchBar)
		
			/** Optional widget to add between the search bar and the table view (e.g. a SBasicFilterBar). */
			SLATE_NAMED_SLOT(FArguments, RowBelowSearchBar)
		
			/** Optional, alternate content to show instead of the tree view when there are no rows. */
			SLATE_NAMED_SLOT(FArguments, NoItemsContent)

		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

		//~ Begin IPropertyTreeView Interface
		virtual void RefreshPropertyData(const TArray<FPropertyAssignmentEntry>& Entries, bool bCanReuseExistingRowItems) override;
		virtual void RequestRefilter() const override { TreeView->RequestRefilter(); }
		virtual void RequestResortForColumn(const FName& ColumnId) override { TreeView->RequestResortForColumn(ColumnId); }
		virtual void RequestScrollIntoView(const FConcertPropertyChain& PropertyChain) override;
		virtual TSharedRef<SWidget> GetWidget() override { return SharedThis(this); }
		//~ Begin IPropertyTreeView Interface

	private:

		/** The tree view displaying the replicated properties */
		TSharedPtr<SReplicationTreeView<FPropertyNodeData>> TreeView;
		
		/** Contains all data. */
		TArray<TSharedPtr<FPropertyNodeData>> PropertyRowData;
		/** The instances which do not have any parents. This acts as the item source for the tree view. */
		TArray<TSharedPtr<FPropertyNodeData>> RootPropertyNodes;
		
		struct FCategoryMetaData
		{
			TSharedRef<FPropertyNodeData> Node;
			/** The widget displayed in the category widget. */
			TSharedRef<ICategoryRow> RowWidgetContent;

			FCategoryMetaData(TSharedRef<FPropertyNodeData> Node, TSharedRef<ICategoryRow> RowWidgetContent)
				: Node(Node)
				, RowWidgetContent(RowWidgetContent) 
			{}
		};
		/**
		 * Inverse map of object to its owning category.
		 * Contains all elements of PropertyRowData which are category nodes.
		 *
		 * The main purpose is to allow reuse of items when refreshing hierarchy (to retain selection state, etc.)
		 * 
		 * Important: when multi-editing, this only maps the context object at index 0.
		 * Use FindCategoryMetaData to retrieve.
		 * 
		 * Empty if ShouldBuildCategories() == false.
		 */
		TMap<TSoftObjectPtr<>, FCategoryMetaData> CategoryNodes;
		/**
		 * Inverse map of PropertyRowData using FPropertyData::GetProperty as key.
		 * Contains all elements of PropertyRowData which are property nodes
		 *
		 * The main purpose is to allow reuse of items when refreshing hierarchy (to retain selection state, etc.)
		 */
		TMap<FConcertPropertyChain, TArray<TSharedPtr<FPropertyNodeData>>> ChainToPropertyDataCache;
		
		/** Equal to FPropertyAssignmentEntry::ContextObjects from the last RefreshPropertyData call. This is used during sorting to determine the relative ordering of categories. */
		TArray<TArray<TSoftObjectPtr<>>> SourceEntriesForSorting;

		/** Optional callback for filtering items. */
		SReplicationTreeView<FPropertyData>::FCustomFilter FilterDelegate;
		/** Optional callback for creating category rows. If unset, no category rows are generated. */
		FCreateCategoryRow CreateCategoryRowDelegate;

		/** @return Whether this UI was configured to show category nodes. */
		bool ShouldBuildCategories() const { return CreateCategoryRowDelegate.IsBound(); }
		
		void RefreshCategoryNodes(const TArray<FPropertyAssignmentEntry>& Entries, TMap<TSoftObjectPtr<>, FCategoryMetaData>& NewCategoryNodes);
		void RefreshPropertyNodes(const TArray<FPropertyAssignmentEntry>& Entries, TMap<FConcertPropertyChain, TArray<TSharedPtr<FPropertyNodeData>>>& NewChainToPropertyDataCache);

		/** Looks in ChainToPropertyDataCache for any a node that is referencing PropertyChain and any of ContextObjects. */
		TSharedPtr<FPropertyNodeData> FindPropertyNode(const FConcertPropertyChain& PropertyChain, TConstArrayView<TSoftObjectPtr<>> ContextObjects);
		/** Find the meta data for this item. */
		FCategoryMetaData* FindCategoryMetaData(const FPropertyNodeData& Item) { return CategoryNodes.Find(Item.GetCategoryData()->GetContextObjects()[0]); }
		const FCategoryMetaData* FindCategoryMetaData(const FPropertyNodeData& Item) const { check(Item.IsCategoryNode()); return CategoryNodes.Find(Item.GetCategoryData()->GetContextObjects()[0]); }

		/** Inits RootPropertyRowData from PropertyRowData. */
		void BuildRootPropertyRowData();
		void GetPropertyRowChildren(TSharedPtr<FPropertyNodeData> ParentNodeData, TFunctionRef<void(TSharedPtr<FPropertyNodeData>)> ProcessChild) const;
		void EnumerateRootProperties(const FCategoryData& ParentCategoryData, TFunctionRef<void(TSharedPtr<FPropertyNodeData>)> ProcessChild) const;
		void EnumerateChildProperties(const FPropertyNodeData& ParentNode, TFunctionRef<void(TSharedPtr<FPropertyNodeData>)> ProcessChild) const;

		/** Filters the item according to FilterDelegate. */
		EItemFilterResult FilterItem(const FPropertyNodeData& PropertyNodeData) const;
		/** Generates a category row if NodeData is a category. */
		TSharedPtr<ITableRow> OverrideRowWidget(
			TSharedPtr<FPropertyNodeData> NodeData,
			const TSharedRef<STableViewBase>& TableViewBase,
			const TReplicationTreeData<FPropertyNodeData>::FGenerateRowArgs& Args
			);
		/** Overrides the sort if one of the nodes is a category */
		EComparisonOverride OverrideIsLessThan(const TSharedPtr<FPropertyNodeData>& Left, const TSharedPtr<FPropertyNodeData>& Right) const;
		/** Overrides the search terms for category nodes. */
		ESearchTermResult OverrideGetSearchTerms(const TSharedPtr<FPropertyNodeData>& NodeData, TArray<FString>& InOutSearchStrings) const;
	};
}

