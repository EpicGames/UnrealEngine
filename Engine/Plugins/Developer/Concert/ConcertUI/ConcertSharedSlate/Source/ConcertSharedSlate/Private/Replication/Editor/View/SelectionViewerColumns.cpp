// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Editor/View/Column/SelectionViewerColumns.h"

#include "ConcertFrontendStyle.h"
#include "Replication/Editor/Model/Data/PropertyData.h"
#include "Replication/Editor/Model/Data/ReplicatedObjectData.h"
#include "Replication/Editor/Model/IReplicationStreamModel.h"
#include "Replication/Editor/Utils/DisplayUtils.h"
#include "Replication/Editor/View/Column/IObjectTreeColumn.h"
#include "Replication/Editor/View/Column/ReplicationColumnsUtils.h"
#include "Replication/PropertyChainUtils.h"

#include "Internationalization/Internationalization.h"
#include "Replication/PropertyResolutionCache.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"
#include "Trace/ConcertTrace.h"

#define LOCTEXT_NAMESPACE "ReplicationObjectColumns"

namespace UE::ConcertSharedSlate::ReplicationColumns::TopLevel
{
	const FName LabelColumnId = TEXT("LabelColumn");
	const FName TypeColumnId = TEXT("TypeColumn");
	const FName NumPropertiesColumnId = TEXT("NumPropertiesColumnId");
	
	FObjectColumnEntry LabelColumn(IObjectNameModel* OptionalNameModel, FGetObjectClass GetObjectClassDelegate)
	{
		class FLabelColumn_Object : public IObjectTreeColumn
		{
		public:
			
			FLabelColumn_Object(IObjectNameModel* OptionalNameModel, FGetObjectClass GetObjectClassDelegate)
				:  OptionalNameModel(OptionalNameModel)
				, GetObjectClassDelegate(MoveTemp(GetObjectClassDelegate))
			{}

			virtual SHeaderRow::FColumn::FArguments CreateHeaderRowArgs() const override
			{
				return SHeaderRow::Column(LabelColumnId)
					.DefaultLabel(LOCTEXT("LabelColumnLabel", "Label"))
					.FillWidth(FConcertFrontendStyle::Get()->GetFloat("Concert.Replication.Object.LabelSize"));
			}
			
			virtual TSharedRef<SWidget> GenerateColumnWidget(const FBuildArgs& InArgs) override
			{
				const FReplicatedObjectData& ObjectData = InArgs.RowItem.RowData;
				const TSoftObjectPtr<>& Object = ObjectData.GetObjectPtr();
				
				const FSlateIcon ClassIcon = GetObjectClassDelegate.IsBound() ? DisplayUtils::GetObjectIcon(GetObjectClassDelegate.Execute(Object)) : FSlateIcon{};
				return SNew(SHorizontalBox)
					.ToolTipText(FText::FromString(Object.ToString()))
					
					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(ClassIcon.GetOptionalIcon())
					]
					
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(6.f, 0.f, 0.f, 0.f)
					[
						SNew(STextBlock)
						.HighlightText(TAttribute<FText>::CreateLambda([HighlightText = InArgs.HighlightText](){ return *HighlightText; }))
						.Text_Lambda([this, ObjectPtr = ObjectData.GetObjectPtr()](){ return GetDisplayText(ObjectPtr); })
					];
			}
			
			virtual void PopulateSearchString(const FObjectTreeRowContext& InItem, TArray<FString>& InOutSearchStrings) const override
			{
				const FReplicatedObjectData& ObjectData = InItem.RowData;
				InOutSearchStrings.Add(
					DisplayUtils::GetObjectDisplayText(ObjectData.GetObjectPtr(), OptionalNameModel).ToString()
					);
			}
			
			virtual bool CanBeSorted() const override { return true; } 
			virtual bool IsLessThan(const FObjectTreeRowContext& Left, const FObjectTreeRowContext& Right) const override
			{
				return GetDisplayText(Left.RowData.GetObjectPtr()).ToString() < GetDisplayText(Right.RowData.GetObjectPtr()).ToString();
			}

		private:
			
			IObjectNameModel* const OptionalNameModel;
			const FGetObjectClass GetObjectClassDelegate;
			
			FText GetDisplayText(const TSoftObjectPtr<>& ObjectPtr) const
			{
				return DisplayUtils::GetObjectDisplayText(ObjectPtr, OptionalNameModel);
			}
		};

		return {
			TReplicationColumnDelegates<FObjectTreeRowContext>::FCreateColumn::CreateLambda([OptionalNameModel, GetObjectClassDelegate = MoveTemp(GetObjectClassDelegate)]()
			{
				return MakeShared<FLabelColumn_Object>(OptionalNameModel, GetObjectClassDelegate);
			}),
			LabelColumnId,
			{ static_cast<int32>(ETopLevelColumnOrder::Label) }
		};
	}
	
	FObjectColumnEntry TypeColumn(FGetObjectClass GetObjectClassDelegate)
	{
		class FTypeColumn_Object : public IObjectTreeColumn
		{
		public:
			
			FTypeColumn_Object(FGetObjectClass GetObjectClassDelegate)
				: GetObjectClassDelegate(MoveTemp(GetObjectClassDelegate))
			{}

			virtual SHeaderRow::FColumn::FArguments CreateHeaderRowArgs() const override
			{
				return SHeaderRow::Column(TypeColumnId)
					.DefaultLabel(LOCTEXT("TypeColumnLabel", "Type")) 
					.FillWidth(FConcertFrontendStyle::Get()->GetFloat("Concert.Replication.Object.TypeWidth"));
			}
			
			virtual TSharedRef<SWidget> GenerateColumnWidget(const FBuildArgs& InArgs) override
			{
				const FSoftClassPath Class = GetClass(InArgs.RowItem.RowData.GetObjectPtr());
				return SNew(SBox)
					.Padding(8, 0, 0, 0) // So the type name text is aligned with the header column text
					[
						SNew(STextBlock)
						.HighlightText(TAttribute<FText>::CreateLambda([HighlightText = InArgs.HighlightText](){ return *HighlightText; }))
						.Text(DisplayUtils::GetObjectTypeText(Class))
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					];
			}
			
			virtual void PopulateSearchString(const FObjectTreeRowContext& InItem, TArray<FString>& InOutSearchStrings) const override
			{
				const FSoftClassPath Class = GetClass(InItem.RowData.GetObjectPtr());
				InOutSearchStrings.Add(DisplayUtils::GetObjectTypeText(Class).ToString());
			}
			
			virtual bool CanBeSorted() const override { return true; } 
			virtual bool IsLessThan(const FObjectTreeRowContext& Left, const FObjectTreeRowContext& Right) const override
			{
				const FSoftClassPath LeftClass = GetClass(Left.RowData.GetObjectPtr());
				const FSoftClassPath RightClass = GetClass(Right.RowData.GetObjectPtr());
				return DisplayUtils::GetObjectTypeText(LeftClass).ToString()
					< DisplayUtils::GetObjectTypeText(RightClass).ToString();
			}

		private:
			
			const FGetObjectClass GetObjectClassDelegate;

			FSoftClassPath GetClass(const TSoftObjectPtr<>& Object) const { return GetObjectClassDelegate.Execute(Object); }
		};

		check(GetObjectClassDelegate.IsBound());
		return {
			TReplicationColumnDelegates<FObjectTreeRowContext>::FCreateColumn::CreateLambda([GetObjectClassDelegate = MoveTemp(GetObjectClassDelegate)]()
			{
				return MakeShared<FTypeColumn_Object>(GetObjectClassDelegate);
			}),
			TypeColumnId,
			{ static_cast<int32>(ETopLevelColumnOrder::Type) }
		};
	}

	FObjectColumnEntry NumPropertiesColumn(const IReplicationStreamModel& Model, ENumPropertiesFlags Flags)
	{
		class FNumPropertiesColumn_Object : public IObjectTreeColumn
		{
		public:
			
			FNumPropertiesColumn_Object(const IReplicationStreamModel& Model UE_LIFETIMEBOUND, ENumPropertiesFlags Flags)
				: Model(Model)
				, Flags(Flags)
			{}

			virtual SHeaderRow::FColumn::FArguments CreateHeaderRowArgs() const override
			{
				return SHeaderRow::Column(NumPropertiesColumnId)
					.DefaultLabel(LOCTEXT("NumPropertyColumnLabel", "# Properties"))
					.FillWidth(FConcertFrontendStyle::Get()->GetFloat("Concert.Replication.Object.NumPropertiesSize"));
			}
			
			virtual TSharedRef<SWidget> GenerateColumnWidget(const FBuildArgs& InArgs) override
			{
				return SNew(SHorizontalBox)
					
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(6.f, 0.f, 0.f, 0.f)
					[
						SNew(STextBlock)
						.HighlightText(TAttribute<FText>::CreateLambda([HighlightText = InArgs.HighlightText](){ return *HighlightText; }))
						.Text_Raw(this, &FNumPropertiesColumn_Object::GetDisplayText, InArgs.RowItem.RowData.GetObjectPath())
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					];
			}
			
			virtual void PopulateSearchString(const FObjectTreeRowContext& InItem, TArray<FString>& InOutSearchStrings) const override
			{
				InOutSearchStrings.Add(
					FString::FromInt(GetNumProperties(InItem.RowData.GetObjectPath()))
					);
			}
			
			virtual bool CanBeSorted() const override { return true; } 
			virtual bool IsLessThan(const FObjectTreeRowContext& Left, const FObjectTreeRowContext& Right) const override
			{
				return GetNumProperties(Left.RowData.GetObjectPath()) < GetNumProperties(Right.RowData.GetObjectPath());
			}

		private:
			
			const IReplicationStreamModel& Model;
			const ENumPropertiesFlags Flags;
			
			FText GetDisplayText(FSoftObjectPath ObjectPath) const
			{
				return FText::AsNumber(GetNumProperties(ObjectPath));
			}

			uint32 GetNumProperties(const FSoftObjectPath& ObjectPath) const
			{
				const uint32 ObjectProperties = Model.GetNumProperties(ObjectPath);
				
				const bool bCountSubobjects = EnumHasAnyFlags(Flags, ENumPropertiesFlags::IncludeSubobjectCounts);
				if (!bCountSubobjects)
				{
					return ObjectProperties;
				}

				uint32 NumSubobjectProperties = 0;
				Model.ForEachSubobject(ObjectPath, [this, &NumSubobjectProperties](const FSoftObjectPath& Child)
				{
					NumSubobjectProperties += Model.GetNumProperties(Child);
					return EBreakBehavior::Continue;
				});

				return ObjectProperties + NumSubobjectProperties;
			}
		};
		
		return {
			TReplicationColumnDelegates<FObjectTreeRowContext>::FCreateColumn::CreateLambda([&Model, Flags]()
			{
				return MakeShared<FNumPropertiesColumn_Object>(Model, Flags);
			}),
			NumPropertiesColumnId,
			{ static_cast<int32>(ETopLevelColumnOrder::NumProperties) }
		};
	}
}

#undef LOCTEXT_NAMESPACE
#define LOCTEXT_NAMESPACE "ReplicationPropertyColumns"

namespace UE::ConcertSharedSlate::ReplicationColumns::Property
{
	const FName ReplicatesColumnId = TEXT("ReplicatedColumn");
	const FName LabelColumnId = TEXT("LabelColumn");
	const FName TypeColumnId = TEXT("TypeColumn");
	
	FPropertyColumnEntry LabelColumn()
	{
		class FLabelColumn_Property : public IPropertyTreeColumn
		{
		public:

			virtual SHeaderRow::FColumn::FArguments CreateHeaderRowArgs() const override
			{
				return SHeaderRow::Column(LabelColumnId)
					.DefaultLabel(LOCTEXT("LabelColumnLabel", "Label"))
					.FillWidth(FConcertFrontendStyle::Get()->GetFloat("Concert.Replication.Property.LabelSize"));
			}
			
			virtual TSharedRef<SWidget> GenerateColumnWidget(const FBuildArgs& InArgs) override
			{
				const FPropertyData& PropertyData = InArgs.RowItem.RowData;
				return SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
					.HighlightText(TAttribute<FText>::CreateLambda([HighlightText = InArgs.HighlightText](){ return *HighlightText; }))
					.Text(DisplayUtils::GetPropertyDisplayText(PropertyCache, PropertyData.GetProperty(), ResolveOrLoadClass(PropertyData)));
			}
			
			virtual void PopulateSearchString(const FPropertyTreeRowContext& InItem, TArray<FString>& InOutSearchStrings) const override
			{
				const FPropertyData& PropertyData = InItem.RowData;
				FString DisplayString = DisplayUtils::GetPropertyDisplayString(PropertyCache, InItem.RowData.GetProperty(), ResolveOrLoadClass(PropertyData));
				InOutSearchStrings.Emplace(MoveTemp(DisplayString));
			}
			
			virtual bool CanBeSorted() const override { return true; } 
			virtual bool IsLessThan(const FPropertyTreeRowContext& Left, const FPropertyTreeRowContext& Right) const override
			{
				SCOPED_CONCERT_TRACE(IsLess_ReplicationLabel)
				return DisplayUtils::GetPropertyDisplayString(PropertyCache, Left.RowData.GetProperty(), ResolveOrLoadClass(Left.RowData))
					< DisplayUtils::GetPropertyDisplayString(PropertyCache, Right.RowData.GetProperty(), ResolveOrLoadClass(Right.RowData));
			}

		private:

			/**
			 * Maps FConcertPropertyChain to FProperty.
			 *
			 * This improves GetPropertyDisplayString performance.
			 * This reduced time spent by ~75% when doing a full tree refresh.
			 *
			 * Mutable because the cache may be mutated but it does not inheritently change the state of this object.
			 */
			mutable ConcertSyncCore::PropertyChain::FPropertyResolutionCache PropertyCache;

			static UClass* ResolveOrLoadClass(const FPropertyData& PropertyData)
			{
				SCOPED_CONCERT_TRACE(ResolveOrLoadClass);
				return PropertyData.GetOwningClassPtr().LoadSynchronous();
			}
		};
		
		return {
			TReplicationColumnDelegates<FPropertyTreeRowContext>::FCreateColumn::CreateLambda([]()
			{
				return MakeShared<FLabelColumn_Property>();
			}),
			LabelColumnId,
			{ static_cast<int32>(EReplicationPropertyColumnOrder::Label) }
		};
	}
	
	FPropertyColumnEntry TypeColumn()
	{
		class FTypeColumn_Property : public IPropertyTreeColumn
		{
		public:

			virtual SHeaderRow::FColumn::FArguments CreateHeaderRowArgs() const override
			{
				return SHeaderRow::Column(TypeColumnId)
					.DefaultLabel(LOCTEXT("TypeColumnLabel", "Type"))
					.FillWidth(FConcertFrontendStyle::Get()->GetFloat("Concert.Replication.Property.TypeSize"));
			}
			
			virtual TSharedRef<SWidget> GenerateColumnWidget(const FBuildArgs& InArgs) override
			{
				const FPropertyData& PropertyData = InArgs.RowItem.RowData;
				return SNew(SBox)
					.Padding(8, 0, 0, 0) // So the type name text is aligned with the header column text
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
						.HighlightText(TAttribute<FText>::CreateLambda([HighlightText = InArgs.HighlightText](){ return *HighlightText; }))
						.Text(GetDisplayText(InArgs.RowItem.RowData))
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					];
			}
			
			virtual void PopulateSearchString(const FPropertyTreeRowContext& InItem, TArray<FString>& InOutSearchStrings) const override
			{
				InOutSearchStrings.Add(GetDisplayText(InItem.RowData).ToString());
			}
			
			virtual bool CanBeSorted() const override { return true; } 
			virtual bool IsLessThan(const FPropertyTreeRowContext& Left, const FPropertyTreeRowContext& Right) const override
			{
				SCOPED_CONCERT_TRACE(IsLess_ReplicationType)
				return GetDisplayText(Left.RowData).ToString() < GetDisplayText(Right.RowData).ToString();
			}

		private:
			
			static FText GetDisplayText(const FPropertyData& Args)
			{
				UClass* Class = Args.GetOwningClassPtr().LoadSynchronous();
				const FProperty* Property = Class ? ConcertSyncCore::PropertyChain::ResolveProperty(*Class, Args.GetProperty()) : nullptr;
				return Property ? FText::FromString(Property->GetCPPType()) : LOCTEXT("Unknown", "Unknown");	
			}
		};
		
		return {
			TReplicationColumnDelegates<FPropertyTreeRowContext>::FCreateColumn::CreateLambda([]()
			{
				return MakeShared<FTypeColumn_Property>();
			}),
			TypeColumnId,
			{ static_cast<int32>(EReplicationPropertyColumnOrder::Type) }
		};
	}
}

#undef LOCTEXT_NAMESPACE