// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertFrontendStyle.h"
#include "TreeItemTraitsInput.h"

#include "Delegates/DelegateCombinations.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Replication/Editor/View/Column/IReplicationTreeColumn.h"

namespace UE::ConcertSharedSlate
{
	/** A row that generates its columns generically via IReplicationTreeColumn. */
	template<typename TListItemType>
	class SReplicationColumnRow : public SMultiColumnTableRow<TSharedPtr<TListItemType>>
	{
		using Super = SMultiColumnTableRow<TSharedPtr<TListItemType>>;
	public:

		using FGetColumn = typename TReplicationTreeData<TListItemType>::FGetColumn;
		using FOverrideColumnWidget = typename TReplicationTreeData<TListItemType>::FOverrideColumnWidget;
		using FGetHoveredRowContent = typename TReplicationTreeData<TListItemType>::FGetHoveredRowContent;
		
		SLATE_BEGIN_ARGS(SReplicationColumnRow)
			: _Style(&FCoreStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row"))
		{}
			/** Used for highlighting the text being searched. */
			SLATE_ARGUMENT(TSharedPtr<FText>, HighlightText)

			/** Gets columns info about a certain column */
			SLATE_EVENT(FGetColumn, ColumnGetter)

			/**
			 * Optional. If the delegate returns non-null, that widget will be used instead of the one the column would generate.
			 * This is useful, e.g. if you want to generate a separator widget between items.
			 */
			SLATE_EVENT(FOverrideColumnWidget, OverrideColumnWidget)
		
			/** Optional. Gets the content to overlay on hovered rows; it covers the entire row. */
			SLATE_EVENT(FGetHoveredRowContent, GetHoveredRowContent)

			/** The data to pass to TReplicationColumn::BuildColumnWidget. */
			SLATE_ARGUMENT(TSharedPtr<TListItemType>, RowData)
		
			/** The name of the column that will have the SExpanderArrow for the tree view. */
			SLATE_ARGUMENT(FName, ExpandableColumnLabel)

			/** Style to use for rows, e.g. for making them alternate in grey */
			SLATE_STYLE_ARGUMENT(FTableRowStyle, Style)
		SLATE_END_ARGS()

		void Construct(
			const FArguments& InArgs,
			TSharedRef<STableViewBase> InOwner)
		{
			ColumnGetterDelegate = InArgs._ColumnGetter;
			OverrideColumnWidgetDelegate = InArgs._OverrideColumnWidget;
			GetHoveredRowContent = InArgs._GetHoveredRowContent;
			HighlightText = InArgs._HighlightText;
			RowData = InArgs._RowData;
			ExpandableColumnLabel = InArgs._ExpandableColumnLabel;

			using FTableRowArgs = typename STableRow<TSharedPtr<TListItemType>>::FArguments;
			SMultiColumnTableRow<TSharedPtr<TListItemType>>::Construct(
				FTableRowArgs()
				.Style(InArgs._Style),
				InOwner
				);
		}
		
		virtual void ConstructChildren(ETableViewMode::Type InOwnerTableMode, const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent) override
		{
			Super::ConstructChildren(InOwnerTableMode, InPadding, InContent);

			if (GetHoveredRowContent.IsBound())
			{
				const auto[HoverContent, HAlign] = GetHoveredRowContent.Execute(RowData);
				this->ChildSlot
				[
					SNew(SOverlay)

					// Generated columns
					+SOverlay::Slot()
					[
						this->ChildSlot.GetWidget()
					]

					// Hovered overlay
					+SOverlay::Slot()
					[
						SNew(SHorizontalBox)
						
						+SHorizontalBox::Slot()
						.HAlign(HAlign)
						[
							SNew(SBox)
							.Visibility_Lambda([this](){ return this->IsHovered() ? EVisibility::Visible : EVisibility::Hidden; })
							[
								HoverContent
							]
						]
					]
				];
			}
		}

		/** Generates the widget representing this row. */
		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			const TSharedPtr<SWidget> ColumnOverride = OverrideColumnWidgetDelegate.IsBound()
				? OverrideColumnWidgetDelegate.Execute(ColumnName, *RowData.Get())
				: nullptr;
			if (ColumnOverride)
			{
				return ColumnOverride.ToSharedRef();
			}
			
			TSharedPtr<IReplicationTreeColumn<TListItemType>> Column = ColumnGetterDelegate.Execute(ColumnName); ensure(Column);
			if (!Column)
			{
				return SNullWidget::NullWidget;
			}
			
			const TSharedRef<SWidget> ColumnWidget = Column->GenerateColumnWidget({ HighlightText, *RowData.Get() });
			const bool bNeedsExpanderArrow = ColumnName == ExpandableColumnLabel;
			if (!bNeedsExpanderArrow)
			{
				return SNew(SBox)
					// Enforce all items to be the same size so it is more consistent with other places, like the SSceneOutliner
					.MaxDesiredHeight(FConcertFrontendStyle::Get()->GetFloat("Concert.Replication.Tree.RowHeight"))
					.VAlign(VAlign_Center)
					[
						ColumnWidget
					];
			}
			
			return SNew(SBox)
				// Enforce all items to be the same size so it is more consistent with other places, like the SSceneOutliner
				.MaxDesiredHeight(FConcertFrontendStyle::Get()->GetFloat("Concert.Replication.Tree.RowHeight"))
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(6, 0, 0, 0)
					[
						SNew(SExpanderArrow, SReplicationColumnRow::SharedThis(this))
						.IndentAmount(12)
					]

					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.VAlign(VAlign_Center)
					[
						ColumnWidget
					]
				];
		}

	private:
		
		FGetColumn ColumnGetterDelegate;
		FOverrideColumnWidget OverrideColumnWidgetDelegate;
		FGetHoveredRowContent GetHoveredRowContent;
		TSharedPtr<FText> HighlightText;
		TSharedPtr<TListItemType> RowData;
		FName ExpandableColumnLabel;
	};
}