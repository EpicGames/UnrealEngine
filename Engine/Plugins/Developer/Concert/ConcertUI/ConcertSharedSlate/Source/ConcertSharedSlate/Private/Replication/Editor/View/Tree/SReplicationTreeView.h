// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Editor/View/Column/ReplicationColumnInfo.h"
#include "Replication/Editor/View/Column/ReplicationColumnsUtils.h"
#include "TreeItemTraitsInput.h"
#include "TreeItemTraits.h"

#include "Algo/RemoveIf.h"
#include "Misc/TextFilter.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/STreeView.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Trace/ConcertTrace.h"

#define LOCTEXT_NAMESPACE "SReplicationListView"

namespace UE::ConcertSharedSlate
{
	enum class EItemFilterResult : uint8
	{
		/** The item passes the filter and should be displayed */
		Include,
		/** The item does not pass the filter and not should be displayed */
		Exclude,
		/** The item is only displayed if the filter passes on one of its children. If no child is included, the item is not displayed. */
		IncludeOnlyIfChildIsIncluded
	};

	/** Result for the IsLessThan override delegate in SReplicationTreeView. */
	enum class EComparisonOverride : uint8
	{
		/** Left < Right is to be treated as true */
		Less,
		/** Left < Right is to be treated as false */
		NotLess,
		/** Use the standard column comparision delegates to find out. */
		UseDefault
	};

	/** Result for the GetSearchString override delegate in SReplicationTreeView. */
	enum class ESearchTermResult : uint8
	{
		/** The delegate wants the passed in item to go through normal text search as well. */
		UseDefault,
		/** The delegate does not want the passed in item to go through the rest of the text search process. */
		UseOverrideOnly,
	};
	
	/**
	 * Shared code for the tree view for replicated actors and properties.
	 * 
	 * It is a tree view that is searchable with a search box and exposes slots to add more filter widgets, such as SBasicFilterBar.
	 * The columns are modular and are abstracted by IReplicationTreeColumn. You can customize row content by specializing TReplicationTreeItemTraits;
	 * the default implementation creates a SReplicationColumnRow, which uses IReplicationTreeColumn to draw the columns. 
	 */
	template<typename TItemType>
	class SReplicationTreeView : public SCompoundWidget
	{
	public:
		
		using FOverrideColumnWidget = typename TReplicationTreeData<TItemType>::FOverrideColumnWidget;
		DECLARE_DELEGATE_OneParam(FDeleteItems, const TArray<TSharedPtr<TItemType>>& SelectedItems);
		DECLARE_DELEGATE(FOnSelectionChanged);

		// TODO UE-216456: The following callbacks could be extracted to a IReplicationItem<TItemType> to simplify SReplicationTreeView implementation 
		using FOverrideRowWidget	= typename TReplicationTreeData<TItemType>::FOverrideRowWidget;
		using FGetHoveredRowContent = typename TReplicationTreeData<TItemType>::FGetHoveredRowContent;
		DECLARE_DELEGATE_RetVal_OneParam(EItemFilterResult, FCustomFilter, const TItemType& Item);
		DECLARE_DELEGATE_RetVal_TwoParams(EComparisonOverride, FIsLessThanOverride, const TSharedPtr<TItemType>& Left, const TSharedPtr<TItemType>& Right);
		DECLARE_DELEGATE_TwoParams(FGetItemChildren, TSharedPtr<TItemType> Item, TFunctionRef<void(TSharedPtr<TItemType>)> ProcessChild);
		DECLARE_DELEGATE_RetVal_TwoParams(ESearchTermResult, FGetSearchTermsOverride, const TSharedPtr<TItemType>& Item, TArray<FString>& InOutSearchTerms);

		enum class EContent
		{
			TreeView,
			Custom
		};

		SLATE_BEGIN_ARGS(SReplicationTreeView<TItemType>)
			: _HeaderRowVisibility(EVisibility::Visible)
			, _SelectionMode(ESelectionMode::Single)
			, _RowStyle(&FCoreStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row"))
		{}
			/** The items to display */
			SLATE_ARGUMENT(TArray<TSharedPtr<TItemType>>*, RootItemsSource)

			/** Gets an items children for the tree view */
			SLATE_EVENT(FGetItemChildren, OnGetChildren)

			/** Optional. Called when the user presses the delete key */
			SLATE_EVENT(FDeleteItems, OnDeleteItems)

			/** Called to generate the context menu for an item */
			SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)

			/** Called when the selection changes. Call GetSelectedItems to get the selected items. */
			SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)

			/** Optional callback to do even more filtering of items. */
			SLATE_EVENT(FCustomFilter, FilterItem)
			/** Optional. Gets the content to overlay on hovered rows; it covers the entire row. */
			SLATE_EVENT(FGetHoveredRowContent, GetHoveredRowContent)
			/**
			 * Optional. If the delegate returns non-null, that widget will be used instead of the one the column would generate.
			 * This is useful, e.g. if you want to generate a separator widget between items.
			 */
			SLATE_EVENT(FOverrideColumnWidget, OverrideColumnWidget)
			/** Optional. The widget returned by this delegate ends up overriding the default row widget. */
			SLATE_EVENT(FOverrideRowWidget, OverrideRowWidget)
			/** Optional. This delegate is used during sorting and can override the default sorting behaviour for some items. This can be used e.g. for category nodes. */
			SLATE_EVENT(FIsLessThanOverride, OverrideIsLessThan)
			/** Optional. Can generate custom terms for an item type. This can be used e.g. for category nodes. */
			SLATE_EVENT(FGetSearchTermsOverride, OverrideGetSearchTerms)
			
			/** The columns this list should have */
			SLATE_ARGUMENT(TArray<TReplicationColumnEntry<TItemType>>, Columns)
			/** The name of the column that will have the SExpanderArrow for the tree view. */
			SLATE_ARGUMENT(FName, ExpandableColumnLabel)
			/** Visibility of the header row */
			SLATE_ARGUMENT(EVisibility, HeaderRowVisibility)
			/** Initial primary sort to set. */
			SLATE_ARGUMENT(FColumnSortInfo, PrimarySort)
			/** Initial secondary sort to set. */
			SLATE_ARGUMENT(FColumnSortInfo, SecondarySort)
		
			/** How many items are to allowed to be selected */
			SLATE_ARGUMENT(ESelectionMode::Type, SelectionMode)

			/** Optional widget to add to the left of the search bar. */
			SLATE_NAMED_SLOT(FArguments, LeftOfSearchBar)
			/** Optional widget to add to the right of the search bar. */
			SLATE_NAMED_SLOT(FArguments, RightOfSearchBar)
			/** Optional widget to add between the search bar and the table view (e.g. a SBasicFilterBar). */
			SLATE_NAMED_SLOT(FArguments, RowBelowSearchBar)

			/** Optional, alternate content to show instead of the tree view when there are no rows. */
			SLATE_NAMED_SLOT(FArguments, NoItemsContent)

			/** Style to use for rows, e.g. for making them alternate in grey */
			SLATE_STYLE_ARGUMENT(FTableRowStyle, RowStyle)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs)
		{
			AllRootItems = InArgs._RootItemsSource;
			check(AllRootItems);

			OnGetChildrenDelegate = InArgs._OnGetChildren;
			OnDeleteItemsDelegate = InArgs._OnDeleteItems;
			CustomFilterDelegate = InArgs._FilterItem;
			GetHoveredRowContentDelegate = InArgs._GetHoveredRowContent;
			OverrideColumnWidgetDelegate = InArgs._OverrideColumnWidget;
			OverrideRowWidgetDelegate = InArgs._OverrideRowWidget;
			OverrideIsLessThanDelegate = InArgs._OverrideIsLessThan;
			OverrideGetSearchTerms = InArgs._OverrideGetSearchTerms;
			ExpandableColumnId = InArgs._ExpandableColumnLabel;
			RowStyle = InArgs._RowStyle;
			
			SearchText = MakeShared<FText>();
			SearchTextFilter = MakeShared<TTextFilter<const TSharedPtr<TItemType>&>>(
				TTextFilter<const TSharedPtr<TItemType>&>::FItemToStringArray::CreateSP(this, &SReplicationTreeView::PopulateSearchStrings)
				);
			SearchTextFilter->OnChanged().AddSP(this, &SReplicationTreeView::RequestRefilter);
			
			ChildSlot
			[
				SNew(SVerticalBox)

				// Search
				+SVerticalBox::Slot()
				.Padding(1.f)
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.Padding(1.f)
					.AutoWidth()
					[
						InArgs._LeftOfSearchBar.Widget
					]

					+SHorizontalBox::Slot()
					.Padding(1.f)
					.FillWidth(1.f)
					[
						SAssignNew(SearchBox, SSearchBox)
						.HintText(LOCTEXT("SearchHint", "Search..."))
						.OnTextChanged(this, &SReplicationTreeView::OnSearchTextChanged)
						.OnTextCommitted(this, &SReplicationTreeView::OnSearchTextCommitted)
						.DelayChangeNotificationsWhileTyping(true)
					]

					+SHorizontalBox::Slot()
					.Padding(1.f)
					.AutoWidth()
					[
						InArgs._RightOfSearchBar.Widget
					]
				]

				// Optional slot between search bar and table, e.g. for an external SBasicFilterBar
				+SVerticalBox::Slot()
				.Padding(1.f)
				.AutoHeight()
				[
					InArgs._RowBelowSearchBar.Widget
				]

				// Table row
				+SVerticalBox::Slot()
				.Padding(1.f)
				.FillHeight(1.f)
				[
					SNew(SScrollBox)
					+SScrollBox::Slot()
					.FillSize(1.f)
					[
						CreateTreeView(InArgs)
					]
				]
			];

			if (!InArgs._PrimarySort.SortedColumnId.IsNone())
			{
				SetPrimarySortMode(
					InArgs._PrimarySort.SortedColumnId,
					// In case None was specified by accident, use good defaults or no sorting will occur
					InArgs._PrimarySort.SortMode == EColumnSortMode::None ? EColumnSortMode::Ascending : InArgs._PrimarySort.SortMode
					);
			}
			if (!InArgs._SecondarySort.SortedColumnId.IsNone())
			{
				SetSecondarySortMode(
					InArgs._SecondarySort.SortedColumnId, 
					// In case None was specified by accident, use good defaults or no sorting will occur
					InArgs._SecondarySort.SortMode == EColumnSortMode::None ? EColumnSortMode::Ascending : InArgs._SecondarySort.SortMode
					);
			}
		}
		
		void RequestRefilter()
		{
			bFilterChanged = true;
			bRequestedSort = true;
		}
		void RequestResort()
		{
			bRequestedSort = true;
		}
		
		/** Requests that the given column be resorted, if it currently affects the row sorting. */
		void RequestResortForColumn(const FName& ColumnId)
		{
			if (PrimarySortInfo.SortedColumnId == ColumnId || SecondarySortInfo.SortedColumnId == ColumnId)
			{
				RequestResort();
			}
		}

		void SetPrimarySortMode(FName SortedColumnId, EColumnSortMode::Type SortMode)
		{
			const TSharedPtr<IReplicationTreeColumn<TItemType>> Column = FindColumnByName(SortedColumnId);
			if (ensureAlways(Column && Column->CanBeSorted()))
			{
				PrimarySortInfo = { SortedColumnId, SortMode };
				
				if (SortedColumnId == SecondarySortInfo.SortedColumnId) // Cannot be primary and secondary at the same time.
				{
					SecondarySortInfo.SortedColumnId = FName();
					SecondarySortInfo.SortMode = EColumnSortMode::None;
				}
			}
		}
		void SetSecondarySortMode(FName SortedColumnId, EColumnSortMode::Type SortMode)
		{
			if (!PrimarySortInfo.IsValid())
			{
				SetPrimarySortMode(SortedColumnId, SortMode);
				return;
			}

			if (PrimarySortInfo.SortedColumnId == SortedColumnId)
			{
				return;
			}
			
			const TSharedPtr<IReplicationTreeColumn<TItemType>> Column = FindColumnByName(SortedColumnId);
			if (ensure(Column && Column->CanBeSorted()))
			{
				SecondarySortInfo = { SortedColumnId, SortMode };
			}
		}
		
		void SetSelectedItems(const TArray<TSharedPtr<TItemType>>& ObjectsToSelect, bool bIsSelected)
		{
			TreeView->ClearSelection();
			TreeView->SetItemSelection(ObjectsToSelect, bIsSelected);
		}
		void SetExpandedItems(const TArray<TSharedPtr<TItemType>>& ObjectsToSelect, bool bIsExpanded)
		{
			for (TSharedPtr<TItemType> Item : ObjectsToSelect)
			{
				TreeView->SetItemExpansion(Item, bIsExpanded);
			}
		}

		void RequestScrollIntoView(const TSharedPtr<TItemType>& Item)
		{
			TreeView->RequestScrollIntoView(Item);
		}
		
		TArray<TSharedPtr<TItemType>> GetSelectedItems() const { return TreeView->GetSelectedItems(); }
		const TArray<TSharedPtr<TItemType>>& GetFilteredRootItems() const { return FilteredRootItems; }
		const FText& GetHighlightText() const { return *SearchText; }
		
		virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
		virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	private:

		/** The widget being used for search. */
		TSharedPtr<SSearchBox> SearchBox;
		/** Used to highlight text in text widgets */
		TSharedPtr<FText> SearchText;
		/** Performs text search */
		TSharedPtr<TTextFilter<const TSharedPtr<TItemType>&>> SearchTextFilter;

		/** ListView's header row */
		TSharedPtr<SHeaderRow> HeaderRow;
		/** Displays the contents */
		TSharedPtr<STreeView<TSharedPtr<TItemType>>> TreeView;
		/** The name of the column which will have the SExpandableArrow widget for the tree view. */
		FName ExpandableColumnId;
		
		/** Binds column names to their infos. */
		TMap<FName, TReplicationColumnEntry<TItemType>> ColumnInfos;
		/**
		 * Columns currently being displayed.
		 * Currently columns are created statically on widget construction but in the future we could add dynamic column registration.
		 * Hence for now every entry in ColumnInfos will have a corresponding entry here.
		 */
		TMap<FName, TSharedRef<IReplicationTreeColumn<TItemType>>> ColumnInstances;

		/** Item source passed to the STreeView. */
		TArray<TSharedPtr<TItemType>>* AllRootItems = nullptr;
		/** Contains only the root items that passed the filters. */
		TArray<TSharedPtr<TItemType>> FilteredRootItems;
		/** Includes ALL items in the hierarchy that have itself passed the filter or one of its children has passed the filter. */
		TSet<TSharedPtr<TItemType>> AllFilteredItems;

		struct FItemMetaData
		{
			/** Whether the item is expanded */
			bool bIsExpanded = false;
		};
		/** Additional data about items needed for misc operations, like expansion. */
		TMap<TSharedPtr<TItemType>, FItemMetaData> ItemMetaData;

		/** Whether all items should be expanded. True while searching and false otherwise. */
		bool bForceParentItemsExpanded = false;

		/** Whether to call ReapplyFilters next Tick(). */
		bool bFilterChanged = false;
		/** Whether to call Resort next Tick(). */
		bool bRequestedSort = false;

		FColumnSortInfo PrimarySortInfo;
		FColumnSortInfo SecondarySortInfo;

		/** Callback for getting an item's children. */
		FGetItemChildren OnGetChildrenDelegate;
		/** Optional delegate for responding to pressing the delete button */
		FDeleteItems OnDeleteItemsDelegate;
		/** Optional delegate for filtering the items even more. */
		FCustomFilter CustomFilterDelegate;
		/** Optional. The content to overlay on hovered rows; it covers the entire row. */
		FGetHoveredRowContent GetHoveredRowContentDelegate;
		/** Optional delegate for overriding the column widgets. */
		FOverrideColumnWidget OverrideColumnWidgetDelegate;
		/** Optional. The widget returned by this delegate ends up overriding the default row widget. */
		FOverrideRowWidget OverrideRowWidgetDelegate;
		/** Optional. This delegate is used during sorting and can override the default sorting behaviour for some items. */
		FIsLessThanOverride OverrideIsLessThanDelegate;
		/** Optional. Can generate custom terms for an item type. This can be used e.g. for category nodes. */
		FGetSearchTermsOverride OverrideGetSearchTerms;

		/** Style to use for rows */
		const FTableRowStyle* RowStyle = nullptr;

		// STreeView creation
		TSharedRef<SWidget> CreateTreeView(const FArguments& InArgs);
		TSharedRef<SHeaderRow> CreateHeaderRow(const FArguments& InArgs);
		TSharedRef<ITableRow> OnGenerateRowWidget(TSharedPtr<TItemType> Item, const TSharedRef<STableViewBase>& OwnerTable);
		void GetRowChildren(TSharedPtr<TItemType> Item, TArray<TSharedPtr<TItemType>>& OutChildren);
		
		// Sorting callbacks
		EColumnSortPriority::Type GetColumnSortPriority(const FName ColumnId) const;
		EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;
		void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);

		// Searching callbacks
		void OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type CommitType);
		void OnSearchTextChanged(const FText& InSearchText);

		// Filtering
		enum class EFilterResult : uint8
		{
			ItemOrChildrenPassFilter,
			NoneInHierarchyPassFilter
		};
		void PopulateSearchStrings(const TSharedPtr<TItemType>& Item, TArray<FString>& OutSearchStrings);
		void ReapplyFilters();
		EFilterResult ApplyFiltersRecursive(const TSharedPtr<TItemType>& Item, TSet<TSharedPtr<TItemType>>& FilteredItemsToShow);
		EItemFilterResult PassesFilters(const TSharedPtr<TItemType>& Item);

		/** Called after RootItems has changed. Removes all invalidated ItemMetaData entries */
		void CleanseItemMetaData();
		/** Applies the cached expansion states to all items */
		void ReapplyExpansionStates();
		/** Callback into tree view when expansion state is changed. */
		void OnItemExpansionChanged(TSharedPtr<TItemType> Item, bool bIsExpanded);

		/** @return Whether currently searching. */
		bool IsSearching() const { return !SearchTextFilter->GetRawFilterText().IsEmpty(); }

		// Sorting
		void Resort();
		void Sort(TArray<TSharedPtr<TItemType>>& Items);
		TSharedPtr<IReplicationTreeColumn<TItemType>> FindColumnByName(const FName& ColumnId) const;
	};

	template <typename TItemType>
	FReply SReplicationTreeView<TItemType>::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		if (InKeyEvent.GetKey() == EKeys::Delete && OnDeleteItemsDelegate.IsBound())
		{
			OnDeleteItemsDelegate.Execute(TreeView->GetSelectedItems());
			return FReply::Handled();
		}
	
		return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
	}

	template <typename TItemType>
	void SReplicationTreeView<TItemType>::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		SCOPED_CONCERT_TRACE(TickReplicationTree);
		
		if (bFilterChanged)
		{
			ReapplyFilters();
		}

		if (bRequestedSort)
		{
			Resort();
		}
		
		SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	}

	template <typename TItemType>
	TSharedRef<SWidget> SReplicationTreeView<TItemType>::CreateTreeView(const FArguments& InArgs)
	{
		TSharedPtr<SVerticalBox> VerticalBox;
		
		const auto GetTreeVisibility = [this](){ return AllRootItems->IsEmpty() || FilteredRootItems.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; };
		const auto GetWarningVisibility = [this](){ return AllRootItems->IsEmpty() || FilteredRootItems.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed; };
		
		TSharedRef<SWidget> Result = SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			.BorderBackgroundColor(FSlateColor(FLinearColor(0.6f, 0.6f, 0.6f)))
			.Padding(0)
			[
				SAssignNew(VerticalBox, SVerticalBox)
				
				+SVerticalBox::Slot()
				.FillHeight(1.f)
				[
					SAssignNew(TreeView, STreeView<TSharedPtr<TItemType>>)
					.Visibility_Lambda(GetTreeVisibility)
					.OnGetChildren(this, &SReplicationTreeView::GetRowChildren)
					.TreeItemsSource(&FilteredRootItems)
					.OnGenerateRow(this, &SReplicationTreeView::OnGenerateRowWidget)
					.OnContextMenuOpening(InArgs._OnContextMenuOpening)
					.OnSelectionChanged_Lambda([OnSelectionChanged = InArgs._OnSelectionChanged](auto, auto){ OnSelectionChanged.ExecuteIfBound(); })
					.OnExpansionChanged(this, &SReplicationTreeView::OnItemExpansionChanged)
					.SelectionMode(InArgs._SelectionMode)
					.AllowOverscroll(EAllowOverscroll::No)
					.HeaderRow(CreateHeaderRow(InArgs))
					// Preserve the selection when the selected item is hidden due to a parent collapsing
					.AllowInvisibleItemSelection(true)
				]

				+SVerticalBox::Slot()
				.FillHeight(1.f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(0.f, 0.f, 0.f, 20.f)
				[
					SNew(SWidgetSwitcher)
					.WidgetIndex_Lambda([this](){ return AllRootItems->IsEmpty() ? 1 : 0; })
					.Visibility_Lambda(GetWarningVisibility)
					+SWidgetSwitcher::Slot() [ SNew(STextBlock).Text(LOCTEXT("AllFiltered", "All items are filtered.")) ]
					+SWidgetSwitcher::Slot()
					[
						InArgs._NoItemsContent.Widget
					]
				]
			];
		
		return Result;
	}

	template <typename TItemType>
	TSharedRef<SHeaderRow> SReplicationTreeView<TItemType>::CreateHeaderRow(const FArguments& InArgs)
	{
		for (const TReplicationColumnEntry<TItemType>& ColumnEntry : InArgs._Columns)
		{
			const FName& ColumnId = ColumnEntry.ColumnId;
			checkf(!ColumnInfos.Contains(ColumnId), TEXT("Duplicate column ID %s"), *ColumnId.ToString());
			ColumnInfos.Add(ColumnId, ColumnEntry);
		}
		
		TArray<FName> ColumnNames;
		ColumnInfos.GenerateKeyArray(ColumnNames);
		ColumnNames.Sort([this](const FName& Left, const FName& Right) { return ColumnInfos[Left].ColumnInfo.SortOrder < ColumnInfos[Right].ColumnInfo.SortOrder; });
		
		HeaderRow = SNew(SHeaderRow).Visibility(InArgs._HeaderRowVisibility);
		for (const FName& ColumnName : ColumnNames)
		{
			check(ColumnInfos[ColumnName].CreateColumn.IsBound());
			TSharedRef<IReplicationTreeColumn<TItemType>> ColumnInstance = ColumnInfos[ColumnName].CreateColumn.Execute();
			
			const SHeaderRow::FColumn::FArguments HeaderRowArgs = ColumnInstance->CreateHeaderRowArgs();
			checkf(HeaderRowArgs._ColumnId == ColumnName, TEXT("CreateHeaderRowArgs returned %s but in the TMap was bound to %s"), *HeaderRowArgs._ColumnId.ToString(), *ColumnName.ToString());
			HeaderRow->AddColumn(HeaderRowArgs);
			
			ColumnInstances.Add(ColumnName, MoveTemp(ColumnInstance));
		}

		return HeaderRow.ToSharedRef();
	}

	template <typename TItemType>
	TSharedRef<ITableRow> SReplicationTreeView<TItemType>::OnGenerateRowWidget(TSharedPtr<TItemType> Item, const TSharedRef<STableViewBase>& OwnerTable)
	{
		const typename SReplicationColumnRow<TItemType>::FGetColumn ColumnGetter =
			SReplicationColumnRow<TItemType>::FGetColumn::CreateSP(this, &SReplicationTreeView::FindColumnByName
			);

		const typename TReplicationTreeData<TItemType>::FGenerateRowArgs RowArgs
		{
			ColumnGetter,
			OverrideColumnWidgetDelegate,
			GetHoveredRowContentDelegate,
			SearchText,
			ExpandableColumnId,
			RowStyle
		};
		const TSharedPtr<ITableRow> OverrideRowWidget = OverrideRowWidgetDelegate.IsBound()
			? OverrideRowWidgetDelegate.Execute(Item, OwnerTable, RowArgs)
			: nullptr;
		
		return OverrideRowWidget
			? OverrideRowWidget.ToSharedRef()
			// Compile-time decide the row that is supposed to be generated
			: TReplicationTreeItemTraits<TItemType>::GenerateRowWidget(Item, OwnerTable, RowArgs);
	}

	template <typename TItemType>
	void SReplicationTreeView<TItemType>::GetRowChildren(TSharedPtr<TItemType> Item, TArray<TSharedPtr<TItemType>>& OutChildren)
	{
		if (OnGetChildrenDelegate.IsBound())
		{
			OnGetChildrenDelegate.Execute(Item, [this, &OutChildren](TSharedPtr<TItemType> ItemToAdd)
			{
				const bool bAllowedByFilter =
					// When we applied the filter, was this item or one of its children allowed?
					AllFilteredItems.Contains(ItemToAdd)
					// Handle case where no filter has been applied, yet. 
					|| PassesFilters(ItemToAdd) != EItemFilterResult::Exclude;
				if (bAllowedByFilter)
				{
					OutChildren.Add(ItemToAdd);
				}
			});
			
			Sort(OutChildren);
		}
	}

	template <typename TItemType>
	EColumnSortPriority::Type SReplicationTreeView<TItemType>::GetColumnSortPriority(const FName ColumnId) const
	{
		if (ColumnId == PrimarySortInfo.SortedColumnId)
		{
			return EColumnSortPriority::Primary;
		}
		if (ColumnId == SecondarySortInfo.SortedColumnId)
		{
			return EColumnSortPriority::Secondary;
		}

		return EColumnSortPriority::Max; // No specific priority.
	}

	template <typename TItemType>
	EColumnSortMode::Type SReplicationTreeView<TItemType>::GetColumnSortMode(const FName ColumnId) const
	{
		if (ColumnId == PrimarySortInfo.SortedColumnId && PrimarySortInfo.IsValid())
		{
			return PrimarySortInfo.SortMode;
		}
		if (ColumnId == SecondarySortInfo.SortedColumnId && SecondarySortInfo.IsValid())
		{
			return SecondarySortInfo.SortMode;
		}
		return EColumnSortMode::None;
	}

	template <typename TItemType>
	void SReplicationTreeView<TItemType>::OnColumnSortModeChanged(
		const EColumnSortPriority::Type SortPriority,
		const FName& ColumnId,
		const EColumnSortMode::Type InSortMode
		)
	{
		const TSharedPtr<IReplicationTreeColumn<TItemType>> Column = FindColumnByName(ColumnId);
		if (!ensure(Column)
			// Cannot bind
			|| !Column->CanBeSorted())
		{
			return;
		}
		
		if (SortPriority == EColumnSortPriority::Primary)
		{
			SetPrimarySortMode(ColumnId, InSortMode);
		}
		else if (SortPriority == EColumnSortPriority::Secondary)
		{
			SetSecondarySortMode(ColumnId, InSortMode);
		}

		RequestResort();
	}

	template <typename TItemType>
	void SReplicationTreeView<TItemType>::OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type CommitType)
	{
		if (!InFilterText.EqualTo(*SearchText))
		{
			OnSearchTextChanged(InFilterText);
		}
	}

	template <typename TItemType>
	void SReplicationTreeView<TItemType>::OnSearchTextChanged(const FText& InSearchText)
	{
		*SearchText = InSearchText;
		SearchTextFilter->SetRawFilterText(InSearchText);
		SearchBox->SetError(SearchTextFilter->GetFilterErrorText());
	}

	template <typename TItemType>
	void SReplicationTreeView<TItemType>::PopulateSearchStrings(const TSharedPtr<TItemType>& Item, TArray<FString>& OutSearchStrings)
	{
		if (OverrideGetSearchTerms.IsBound()
			&& OverrideGetSearchTerms.Execute(Item, OutSearchStrings) == ESearchTermResult::UseOverrideOnly)
		{
			return;
		}
		
		for (const TPair<FName, TSharedRef<IReplicationTreeColumn<TItemType>>>& ColumnEntry : ColumnInstances)
		{
			ColumnEntry.Value->PopulateSearchString(*Item, OutSearchStrings);
		}
	}

	template <typename TItemType>
	void SReplicationTreeView<TItemType>::ReapplyFilters()
	{
		SCOPED_CONCERT_TRACE(ReapplyFilters);
		
		// Try preserving the selected activity.
		TArray<TSharedPtr<TItemType>> SelectedItems = TreeView->GetSelectedItems();

		// Reset the list of displayed activities.
		FilteredRootItems.Reset(AllRootItems->Num());
		AllFilteredItems.Reset();

		// Apply the filter.
		for (const TSharedPtr<TItemType>& Activity : *AllRootItems)
		{
			if (ApplyFiltersRecursive(Activity, AllFilteredItems) == EFilterResult::ItemOrChildrenPassFilter)
			{
				FilteredRootItems.Add(Activity);
			}
		}

		// Restore/reset the selected activity.
		SelectedItems.SetNum(Algo::RemoveIf(SelectedItems, [this](const TSharedPtr<TItemType>& Item){ return !FilteredRootItems.Contains(Item); }));
		if (!SelectedItems.IsEmpty())
		{
			TreeView->SetItemSelection(SelectedItems, true); // Restore previous selection.
			TreeView->RequestScrollIntoView(SelectedItems[0]);
		}

		// Update expansion after items have potentially changed
		CleanseItemMetaData();
		ReapplyExpansionStates();

		bFilterChanged = false;
		RequestResort();
		TreeView->RequestListRefresh();
	}

	template <typename TItemType>
	typename SReplicationTreeView<TItemType>::EFilterResult SReplicationTreeView<TItemType>::ApplyFiltersRecursive(const TSharedPtr<TItemType>& Item, TSet<TSharedPtr<TItemType>>& FilteredItemsToShow)
	{
		bool bPassesAtLeastOnce = false;
		const EItemFilterResult ItemFilterResult = PassesFilters(Item);

		bool bChildPassedAtLeastOnce = false;
		if (OnGetChildrenDelegate.IsBound())
		{
			OnGetChildrenDelegate.Execute(Item, [this, &FilteredItemsToShow, &bChildPassedAtLeastOnce](TSharedPtr<TItemType> ItemToAdd)
			{
				bChildPassedAtLeastOnce |= ApplyFiltersRecursive(ItemToAdd, FilteredItemsToShow) == EFilterResult::ItemOrChildrenPassFilter;
			});
			bPassesAtLeastOnce = bChildPassedAtLeastOnce;
		}

		bPassesAtLeastOnce |= ItemFilterResult == EItemFilterResult::Include
			 || (ItemFilterResult == EItemFilterResult::IncludeOnlyIfChildIsIncluded && bChildPassedAtLeastOnce);
		if (bPassesAtLeastOnce)
		{
			FilteredItemsToShow.Add(Item);
		}

		return bPassesAtLeastOnce ? EFilterResult::ItemOrChildrenPassFilter : EFilterResult::NoneInHierarchyPassFilter;
	}

	template <typename TItemType>
	EItemFilterResult SReplicationTreeView<TItemType>::PassesFilters(const TSharedPtr<TItemType>& Item)
	{
		if (SearchTextFilter->PassesFilter(Item))
		{
			return CustomFilterDelegate.IsBound()
				? CustomFilterDelegate.Execute(*Item)
				: EItemFilterResult::Include; 
		}
		return EItemFilterResult::Exclude;
	}

	template <typename TItemType>
	void SReplicationTreeView<TItemType>::CleanseItemMetaData()
	{
		// While searching, all items are force expanded.
		// Do not remove items from ItemMetaData while searching because we want to restore the expansion states after search is done.
		if (IsSearching())
		{
			return;
		}
		
		TMap<TSharedPtr<TItemType>, FItemMetaData> NewItemMetaData;
		for (const TSharedPtr<TItemType>& Item : AllFilteredItems)
		{
			NewItemMetaData.Add(Item, ItemMetaData.FindOrAdd(Item));
		}
		ItemMetaData = MoveTemp(NewItemMetaData);
	}

	template <typename TItemType>
	void SReplicationTreeView<TItemType>::ReapplyExpansionStates()
	{
		// While searching, expand all items
		bForceParentItemsExpanded = IsSearching();

		for (const TSharedPtr<TItemType> Item : AllFilteredItems)
		{
			if (const FItemMetaData* ItemInfo = ItemMetaData.Find(Item))
			{
				TreeView->SetItemExpansion(Item, ItemInfo->bIsExpanded || bForceParentItemsExpanded);
			}
			else
			{
				TreeView->SetItemExpansion(Item, bForceParentItemsExpanded);
			}
		}
	}

	template <typename TItemType>
	void SReplicationTreeView<TItemType>::OnItemExpansionChanged(TSharedPtr<TItemType> Item, bool bIsExpanded)
	{
		// We don't want to be overriding the cached state while everything is expanded
		if (bForceParentItemsExpanded)
		{
			return;
		}
		
		ItemMetaData.FindOrAdd(Item).bIsExpanded = bIsExpanded;
		if (!bIsExpanded || !OnGetChildrenDelegate.IsBound())
		{
			return;
		}
		
		// Also expand its children
		OnGetChildrenDelegate.Execute(Item, [this](const TSharedPtr<TItemType>& Child)
		{
			if (const FItemMetaData* ChildMetaData = ItemMetaData.Find(Child))
			{
				TreeView->SetItemExpansion(Child, ChildMetaData->bIsExpanded);
			}
		});
	}

	template <typename TItemType>
	void SReplicationTreeView<TItemType>::Resort()
	{
		SCOPED_CONCERT_TRACE(Resort);
		
		Sort(FilteredRootItems);
		// GetRowChildren will be called again, which will do the resort the children.
		TreeView->RequestListRefresh();
	}

	template <typename TItemType>
	void SReplicationTreeView<TItemType>::Sort(TArray<TSharedPtr<TItemType>>& Items)
	{
		SCOPED_CONCERT_TRACE(SortReplicationTree);
		
		const auto IsLessThan = [this](const TSharedPtr<TItemType>& Left, const TSharedPtr<TItemType>& Right, const FName& ColumnName, EColumnSortMode::Type SortMode)
		{
			const TSharedPtr<IReplicationTreeColumn<TItemType>> Column = FindColumnByName(ColumnName);
			if (!ensure(Column) || !ensureMsgf(Column->CanBeSorted(), TEXT("Validate why invariant was broken.")))
			{
				return false;
			}

			switch (SortMode)
			{
				case EColumnSortMode::Ascending: return TReplicationTreeItemTraits<TItemType>::IsLessThan(*Left, *Right, *Column);
				case EColumnSortMode::Descending: return TReplicationTreeItemTraits<TItemType>::IsLessThan(*Right, *Left, *Column);
				case EColumnSortMode::None:
				default: return false;
			};
		};
		
		Items.Sort([this, &IsLessThan](const TSharedPtr<TItemType>& Left, const TSharedPtr<TItemType>& Right)
		{
			const EComparisonOverride ComparisonOverride = OverrideIsLessThanDelegate.IsBound() ? OverrideIsLessThanDelegate.Execute(Left, Right) : EComparisonOverride::UseDefault;
			if (ComparisonOverride != EComparisonOverride::UseDefault)
			{
				return ComparisonOverride == EComparisonOverride::Less;
			}
			
			if (PrimarySortInfo.IsValid() && IsLessThan(Left, Right, PrimarySortInfo.SortedColumnId, PrimarySortInfo.SortMode))
			{
				return true; // Left comes before Right
			}
			// Check for equality by inverting
			if (PrimarySortInfo.IsValid() && IsLessThan(Right, Left, PrimarySortInfo.SortedColumnId, PrimarySortInfo.SortMode))
			{
				return false; // Right comes before Left
			}
			
			// Lhs == Rhs on the primary column, need to order according to the secondary column if one is set.
			return !SecondarySortInfo.IsValid()
				? false
				: IsLessThan(Left, Right, SecondarySortInfo.SortedColumnId, SecondarySortInfo.SortMode);
		});

		bRequestedSort = false;
	}

	template <typename TItemType>
	TSharedPtr<IReplicationTreeColumn<TItemType>> SReplicationTreeView<TItemType>::FindColumnByName(const FName& ColumnId) const
	{
		const TSharedRef<IReplicationTreeColumn<TItemType>>* Instance = ColumnInstances.Find(ColumnId);
		return Instance ? Instance->ToSharedPtr() : nullptr;
	}
}

#undef LOCTEXT_NAMESPACE