// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SReplicationColumnRow.h"
#include "TreeItemTraitsInput.h"

#include "Templates/SharedPointer.h"

namespace UE::ConcertSharedSlate
{
	/** Re-usable default behavior for implementing TReplicationTreeItemTraits::GenerateRowWidget. Simply generates a SReplicationColumnRow. */
	template<typename TItemType>
	TSharedRef<ITableRow> GenerateRowWidget_Default(TSharedPtr<TItemType> Item, const TSharedRef<STableViewBase>& OwnerTable, const typename TReplicationTreeData<TItemType>::FGenerateRowArgs& AdditionalArgs)
	{
		return SNew(SReplicationColumnRow<TItemType>, OwnerTable)
			.HighlightText(AdditionalArgs.HighlightText)
			.ColumnGetter(AdditionalArgs.GetColumnDelegate)
			.OverrideColumnWidget(AdditionalArgs.OverrideColumnWidgetDelegate)
			.GetHoveredRowContent(AdditionalArgs.GetHoveredRowContent)
			.RowData(Item)
			.ExpandableColumnLabel(AdditionalArgs.ExpandableColumnId)
			.Style(AdditionalArgs.RowStyle);
	}

	/** Re-usable default behaviour for implementing sorting items by column. */
	template<typename TItemType>
	bool IsLessThan_Default(const TItemType& Left, const TItemType& Right, const IReplicationTreeColumn<TItemType>& SortByColumn)
	{
		return SortByColumn.IsLessThan(Left, Right);
	}

	/**
	 * Allows type-specific overriding of SReplicationColumnRow behavior.
	 * This effectively implements the Strategy design pattern.
	 */
	template<typename TItemType>
	class TReplicationTreeItemTraits
	{
	public:

		/** Called by SReplicationColumnRow to generate an item's row. */
		static TSharedRef<ITableRow> GenerateRowWidget(TSharedPtr<TItemType> Item, const TSharedRef<STableViewBase>& OwnerTable, const typename TReplicationTreeData<TItemType>::FGenerateRowArgs& AdditionalArgs)
		{
			return GenerateRowWidget_Default(Item, OwnerTable, AdditionalArgs);
		}

		/** Called by SReplicationColumnRow when sorting items by a column (this is the primary or secondary). Implement this to account for custom row types, if any. You usually want to implement this if you implement GenerateRowWidget. */
		static bool IsLessThan(const TItemType& Left, const TItemType& Right, const IReplicationTreeColumn<TItemType>& SortByColumn)
		{
			return IsLessThan_Default(Left, Right, SortByColumn);
		}
	};
}