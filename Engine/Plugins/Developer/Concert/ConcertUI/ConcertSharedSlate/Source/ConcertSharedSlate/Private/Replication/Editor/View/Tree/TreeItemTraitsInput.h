// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"

class FName;
class ITableRow;
class STableViewBase;
class SWidget;

struct FTableRowStyle;

namespace UE::ConcertSharedSlate
{
	template<typename TTreeItemType>
	class IReplicationTreeColumn;

	struct FHoverRowContent
	{
		TSharedRef<SWidget> Widget;
		EHorizontalAlignment Alignment = HAlign_Right;
	};
	
	/** Holds type definitions that server as input to TReplicationTreeItemTraits (and thus should not be template specialized).  */
	template<typename TItemType>
	class TReplicationTreeData
	{
	public:
		
		DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<IReplicationTreeColumn<TItemType>>, FGetColumn, const FName& ColumnId);
		DECLARE_DELEGATE_RetVal_TwoParams(TSharedPtr<SWidget>, FOverrideColumnWidget, const FName& ColumnName, const TItemType& RowData);
		DECLARE_DELEGATE_RetVal_OneParam(FHoverRowContent, FGetHoveredRowContent, const TSharedPtr<TItemType>& Item);
		
		struct FGenerateRowArgs
		{
			/** Gets info about a replication column. */
			FGetColumn GetColumnDelegate;
			/** Overrides a column's content widget. */
			FOverrideColumnWidget OverrideColumnWidgetDelegate;
			
			/** Optional. Gets the content to overlay on hovered rows; it covers the entire row. */
			FGetHoveredRowContent GetHoveredRowContent;

			/** The text to highlight - equal to search text. */
			TSharedPtr<FText> HighlightText;
			/** The name of the column which will have the SExpandableArrow widget for the tree view. */
			FName ExpandableColumnId;
			/** Style to use for rows */
			const FTableRowStyle* RowStyle = nullptr;
		};

		/** Overrides the widget for the row if the delegate returns non-nullptr. */
		DECLARE_DELEGATE_RetVal_ThreeParams(TSharedPtr<ITableRow>, FOverrideRowWidget, TSharedPtr<TItemType> Item, const TSharedRef<STableViewBase>& OwnerTable, const FGenerateRowArgs& AdditionalArgs);
	};
}