// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/ReplicationWidgetFactories.h"

#include "Editor/Model/GenericReplicationStreamModel.h"
#include "Editor/Model/ReadableReplicationStreamModel.h"
#include "Editor/View/MultiEditor/SMultiReplicationStreamEditor.h"
#include "Editor/View/ObjectEditor/SBaseReplicationStreamEditor.h"
#include "Editor/View/Property/SMultiObjectAssignment.h"
#include "Editor/View/Property/SPerObjectPropertyAssignment.h"
#include "Editor/View/Property/SPropertyTreeView.h"
#include "Replication/PropertyAssignmentViewFactory.h"
#include "Replication/Editor/Model/Object/IObjectNameModel.h"
#include "Replication/Editor/View/IPropertyAssignmentView.h"

#include "Widgets/Text/STextBlock.h"

namespace UE::ConcertSharedSlate
{
	TSharedRef<IReplicationStreamModel> CreateReadOnlyStreamModel(TAttribute<const FConcertObjectReplicationMap*> ReplicationMapAttribute)
	{
		return MakeShared<FReadableReplicationStreamModel>(MoveTemp(ReplicationMapAttribute));
	}

	TSharedRef<IEditableReplicationStreamModel> CreateBaseStreamModel(
		TAttribute<FConcertObjectReplicationMap*> ReplicationMapAttribute,
		TSharedPtr<IStreamExtender> Extender
		)
	{
		return MakeShared<FGenericReplicationStreamModel>(MoveTemp(ReplicationMapAttribute), MoveTemp(Extender));
	}
	
	TSharedRef<IReplicationStreamEditor> CreateBaseStreamEditor(FCreateEditorParams EditorParams, FCreateViewerParams ViewerParams)
	{
		return SNew(SBaseReplicationStreamEditor, EditorParams.DataModel, EditorParams.ObjectSource, EditorParams.PropertySource)
			.PropertyAssignmentView(MoveTemp(ViewerParams.PropertyAssignmentView))
			.ObjectColumns(MoveTemp(ViewerParams.ObjectColumns))
			.PrimaryObjectSort(ViewerParams.PrimaryObjectSort)
			.SecondaryObjectSort(ViewerParams.SecondaryObjectSort)
			.ObjectHierarchy(MoveTemp(ViewerParams.ObjectHierarchy))
			.NameModel(MoveTemp(ViewerParams.NameModel))
			.OnExtendObjectsContextMenu(MoveTemp(ViewerParams.OnExtendObjectsContextMenu))
			.OnPreAddSelectedObjectsDelegate(MoveTemp(EditorParams.OnPreAddSelectedObjectsDelegate))
			.OnPostAddSelectedObjectsDelegate(MoveTemp(EditorParams.OnPostAddSelectedObjectsDelegate))
			.ShouldDisplayObject(MoveTemp(ViewerParams.ShouldDisplayObjectDelegate))
			.MakeObjectRowOverlayWidget(MoveTemp(ViewerParams.MakeObjectRowOverlayWidgetDelegate))
			.ObjectOverlayAlignment(ViewerParams.OverlayWidgetAlignment)
			.LeftOfObjectSearchBar() [ MoveTemp(ViewerParams.LeftOfObjectSearchBar.Widget) ]
			.RightOfObjectSearchBar() [ MoveTemp(ViewerParams.RightOfObjectSearchBar.Widget) ]
			.IsEditingEnabled(MoveTemp(EditorParams.IsEditingEnabled))
			.EditingDisabledToolTipText(MoveTemp(EditorParams.EditingDisabledToolTipText))
			.WrapOutliner(MoveTemp(ViewerParams.WrapOutlinerWidgetDelegate));
	}

	TSharedRef<IPropertyTreeView> CreateSearchablePropertyTreeView(FCreatePropertyTreeViewParams Params)
	{
		// The label column is always required
		const bool bHasLabel = Params.PropertyColumns.ContainsByPredicate([](const FPropertyColumnEntry& Entry)
		{
			return Entry.ColumnId == ReplicationColumns::Property::LabelColumnId;
		});
		if (!bHasLabel)
		{
			Params.PropertyColumns.Add(ReplicationColumns::Property::LabelColumn());
		}

		SReplicationTreeView<FPropertyData>::FCustomFilter FilterDelegate = Params.FilterItem.IsBound()
			? SReplicationTreeView<FPropertyData>::FCustomFilter::CreateLambda([Filter = MoveTemp(Params.FilterItem)](const FPropertyData& Item)
			{
				return Filter.Execute(Item) == EFilterResult::PassesFilter
					? EItemFilterResult::Include
					: EItemFilterResult::Exclude;
			})
			: SReplicationTreeView<FPropertyData>::FCustomFilter{};
		
		return SNew(SPropertyTreeView)
			.FilterItem(MoveTemp(FilterDelegate))
			.CreateCategoryRow(MoveTemp(Params.CreateCategoryRow))
			.Columns(MoveTemp(Params.PropertyColumns))
			.ExpandableColumnLabel(ReplicationColumns::Property::LabelColumnId)
			.PrimarySort(Params.PrimaryPropertySort)
			.SecondarySort(Params.SecondaryPropertySort)
			.SelectionMode(ESelectionMode::Multi)
			.LeftOfSearchBar() [ MoveTemp(Params.LeftOfPropertySearchBar.Widget) ]
			.RightOfSearchBar() [ MoveTemp(Params.RightOfPropertySearchBar.Widget) ]
			.RowBelowSearchBar() [ MoveTemp(Params.RowBelowSearchBar.Widget) ]
			.NoItemsContent() [ MoveTemp(Params.NoItemsContent.Widget) ];
	}

	TSharedRef<IPropertyAssignmentView> CreatePerObjectAssignmentView(FCreatePerObjectAssignmentViewParams Params)
	{
		return SNew(SPerObjectPropertyAssignment, Params.PropertyTreeView)
			.PropertySource(Params.PropertySource);
	}

	TSharedRef<IMultiObjectPropertyAssignmentView> CreateMultiObjectAssignmentView(FCreateMultiObjectAssignmentViewParams Params)
	{
		return SNew(SMultiObjectAssignment, Params.PropertyTreeView)
			.PropertySource(Params.PropertySource)
			.ObjectHierarchy(Params.ObjectHierarchy);
	}

	TSharedRef<IMultiReplicationStreamEditor> CreateBaseMultiStreamEditor(FCreateMultiStreamEditorParams EditorParams, FCreateViewerParams ViewerParams)
	{
		return SNew(SMultiReplicationStreamEditor, MoveTemp(EditorParams), MoveTemp(ViewerParams));
	}
}

