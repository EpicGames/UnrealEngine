// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConcertScrollBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "SConcertScrollBox"

void SConcertScrollBox::Construct(const FArguments& InArgs)
{
	bIsLocked = true;
	bPreventLock = false;

	ScrollBar = SNew(SScrollBar)
		.Thickness(FVector2D(12.0f, 12.0f));

	ChildSlot
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()	
			[
 				SAssignNew(ScrollBox, SScrollBox)
 				.ExternalScrollbar(ScrollBar)
				.OnUserScrolled(this, &SConcertScrollBox::HandleUserScrolled)
			]
		
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.VAlign(VAlign_Bottom)
				.AutoHeight()
				[
					CreateScrollBarButton(LOCTEXT("ScrollToStartToolTip", "Scroll to the start of the list."), TEXT("\xf077"), FOnClicked::CreateSP(this, &SConcertScrollBox::HandleScrollToStart)) 
				]

				+SVerticalBox::Slot()
				.FillHeight(1.f)
				[
					ScrollBar.ToSharedRef()
				]

				+SVerticalBox::Slot()
				.VAlign(VAlign_Bottom)
				.AutoHeight()
				[
					CreateScrollBarButton(LOCTEXT("ScrollToEndToolTip", "Scroll to the end of the list."), TEXT("\xf078"), FOnClicked::CreateSP(this, &SConcertScrollBox::HandleScrollToEnd))
				]
			]
		];

	// Forward the slots to the inner scroll box.
	for (const SConcertScrollBox::FSlot::FSlotArguments& InSlot : InArgs._Slots)
	{
		if (InSlot.GetAttachedWidget())
		{
			ScrollBox->AddSlot()
			[
				InSlot.GetAttachedWidget().ToSharedRef()
			];
		}
	}
}

void SConcertScrollBox::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (IsAtBottom() && !bPreventLock)
	{
		bIsLocked = true;
	}

	if (bIsLocked)
	{
		ScrollBox->ScrollToEnd();
	};

	/** Locking was prevented for one tick to allow the scrollbar to move without triggering a new lock. */
	if (bPreventLock)
	{
		bPreventLock = false;
	}
}


SConcertScrollBox::FSlot::FSlotArguments SConcertScrollBox::Slot()
{
	return SConcertScrollBox::FSlot::FSlotArguments(MakeUnique<SConcertScrollBox::FSlot>());
}

void SConcertScrollBox::HandleUserScrolled(float)
{
	bPreventLock = true;
	bIsLocked = false;
}

bool SConcertScrollBox::IsAtBottom()
{
	return ScrollBar->DistanceFromBottom() == 0.f;
}

FReply SConcertScrollBox::HandleScrollToStart()
{
	bIsLocked = false;
	bPreventLock = true;
	ScrollBox->ScrollToStart();
	return FReply::Handled();
}

FReply SConcertScrollBox::HandleScrollToEnd()
{
	bIsLocked = true;
	ScrollBox->ScrollToEnd();
	return FReply::Handled();
}

TSharedRef<SButton> SConcertScrollBox::CreateScrollBarButton(const FText& InToolTip, const FString& InIcon, FOnClicked OnClickedDelegate) const
{
	return SNew(SButton)
		.Visibility(this, &SConcertScrollBox::HandleScrollButtonsVisibility)
		.ToolTipText(InToolTip)
		.ButtonStyle(FAppStyle::Get(), "FlatButton.Default")
		.ForegroundColor(FLinearColor::White)
		.OnClicked(OnClickedDelegate)
		.ContentPadding(FMargin(2.f, 2.f))
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
				.Text(FText::FromString(InIcon)) /*fa-chevron-up*/
				.Justification(ETextJustify::Center)
			]
		];
}

EVisibility SConcertScrollBox::HandleScrollButtonsVisibility() const
{
	return ScrollBar->GetVisibility();
}

#undef LOCTEXT_NAMESPACE /** SConcertScrollBox */
