// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConcertSessionRecovery.h"

#include "ConcertSyncSessionTypes.h"
#include "Widgets/SWindow.h"
#include "Session/Activity/ActivityColumn.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "SNegativeActionButton.h"
#include "SPositiveActionButton.h"

#include "Session/Activity/PredefinedActivityColumns.h"

#define LOCTEXT_NAMESPACE "SConcertSessionRecovery"

void SConcertSessionRecovery::Construct(const FArguments& InArgs)
{
	IntroductionText = InArgs._IntroductionText;
	ParentWindow = InArgs._ParentWindow;
	OnRestoreFn = InArgs._OnRestore;
	OnCancelFn = InArgs._OnCancel;
	IsRecoverThroughButtonVisible = InArgs._IsRecoverThroughButtonsVisible;

	ActivityViewOptions = MakeShared<FConcertSessionActivitiesOptions>();
	ActivityViewOptions->bEnableConnectionActivityFiltering = InArgs._IsConnectionActivityFilteringEnabled;
	ActivityViewOptions->bEnableLockActivityFiltering = InArgs._IsLockActivityFilteringEnabled;
	ActivityViewOptions->bEnablePackageActivityFiltering = InArgs._IsPackageActivityFilteringEnabled;
	ActivityViewOptions->bEnableTransactionActivityFiltering = InArgs._IsTransactionActivityFilteringEnabled;
	ActivityViewOptions->bEnableReplicationActivityFiltering = InArgs._IsReplicationActivityFilteringEnabled;
	ActivityViewOptions->bEnableIgnoredActivityFiltering = InArgs._IsIgnoredActivityFilteringEnabled;
	
	SAssignNew(ActivityView, SConcertSessionActivities)
		.OnFetchActivities(InArgs._OnFetchActivities)
		.OnMapActivityToClient(InArgs._OnMapActivityToClient)
		.OnMakeColumnOverlayWidget_Lambda([this](TWeakPtr<SMultiColumnTableRow<TSharedPtr<FConcertSessionActivity>>>, TWeakPtr<FConcertSessionActivity> Activity, const FName& ColumnId) { return MakeRecoverThroughWidget(Activity, ColumnId); })
		.Columns(MakeColumns(InArgs))
		.HighlightText(this, &SConcertSessionRecovery::HighlightSearchText)
		.TimeFormat(ActivityViewOptions.Get(), &FConcertSessionActivitiesOptions::GetTimeFormat)
		.ConnectionActivitiesVisibility(ActivityViewOptions.Get(), &FConcertSessionActivitiesOptions::GetConnectionActivitiesVisibility)
		.LockActivitiesVisibility(ActivityViewOptions.Get(), &FConcertSessionActivitiesOptions::GetLockActivitiesVisibility)
		.PackageActivitiesVisibility(ActivityViewOptions.Get(), &FConcertSessionActivitiesOptions::GetPackageActivitiesVisibility)
		.TransactionActivitiesVisibility(ActivityViewOptions.Get(), &FConcertSessionActivitiesOptions::GetTransactionActivitiesVisibility)
		.ReplicationActivitiesVisibility(ActivityViewOptions.Get(), &FConcertSessionActivitiesOptions::GetReplicationActivitiesVisibility)
		.IgnoredActivitiesVisibility(ActivityViewOptions.Get(), &FConcertSessionActivitiesOptions::GetIgnoredActivitiesVisibility)
		.DetailsAreaVisibility(InArgs._DetailsAreaVisibility)
		.NoActivitiesReasonText(InArgs._NoActivitiesReasonText);

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.BorderBackgroundColor(FSlateColor(FLinearColor(0.6f, 0.6f, 0.6f)))
		.Padding(0)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.FillHeight(1.0)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)

					// Contextual introduction.
					+SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Center)
					.Padding(0, 6)
					[
						SNew(STextBlock)
						.Text(IntroductionText)
						.Visibility(IntroductionText.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible)
					]

					// Search bar.
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0, 1, 0, 2)
					[
						SNew(SHorizontalBox)

						+SHorizontalBox::Slot()
						.FillWidth(1.f)
						[
							SAssignNew(SearchBox, SSearchBox)
							.HintText(LOCTEXT("SearchHint", "Search..."))
							.OnTextChanged(this, &SConcertSessionRecovery::OnSearchTextChanged)
							.OnTextCommitted(this, &SConcertSessionRecovery::OnSearchTextCommitted)
							.DelayChangeNotificationsWhileTyping(true)
						]
						
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							ActivityViewOptions->MakeViewOptionsComboButton()
						]
					]

					// Activity List
					+SVerticalBox::Slot()
					[
						ActivityView.ToSharedRef()
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2, 2)
					[
						SNew(SSeparator)
					]

					// Status bar/View options.
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(4, 2)
					[
						ActivityViewOptions->MakeStatusBar(
							TAttribute<int32>(ActivityView.Get(), &SConcertSessionActivities::GetTotalActivityNum),
							TAttribute<int32>(ActivityView.Get(), &SConcertSessionActivities::GetDisplayedActivityNum)
							)
					]
				]
			]

			// Buttons
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(0, 6)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FMargin(2.0f, 0.0f))
				.Visibility(InArgs._AreRecoverAllAndCancelButtonsVisible.Get() ? EVisibility::Visible : EVisibility::Collapsed)

				+SUniformGridPanel::Slot(0, 0)
				[
					SNew(SPositiveActionButton)
					.ToolTipText(this, &SConcertSessionRecovery::GetRecoverAllButtonTooltip)
					.Text(LOCTEXT("RecoverAll", "Recover All"))
					.Icon(FAppStyle::GetBrush("Icons.CircleArrowRight"))
					.OnClicked(this, &SConcertSessionRecovery::OnRecoverAllClicked)
				]

				+SUniformGridPanel::Slot(1, 0)
				[
					SNew(SNegativeActionButton)
					.ToolTipText(LOCTEXT("CancelRecoveryTooltip", "Discard any recoverable data for your assets and continue with their last saved state"))
					.Text(LOCTEXT("Cancel", "Cancel"))
					.OnClicked(this, &SConcertSessionRecovery::OnCancelRecoveryClicked)
				]
			]
		]
	];
}

TArray<FActivityColumn> SConcertSessionRecovery::MakeColumns(const FArguments& InArgs) const
{
	using namespace UE::ConcertSharedSlate;
	TArray<FActivityColumn> Result;

	if (InArgs._WithClientAvatarColorColumn)
	{
		Result.Add(ActivityColumn::AvatarColor());
	}
	if (InArgs._WithClientNameColumn)
	{
		Result.Add(ActivityColumn::ClientName());
	}
	if (InArgs._WithOperationColumn)
	{
		Result.Add(ActivityColumn::Operation());
	}
	if (InArgs._WithPackageColumn)
	{
		Result.Add(ActivityColumn::Package());
	}
	
	return Result;
}

TSharedPtr<SWidget> SConcertSessionRecovery::MakeRecoverThroughWidget(TWeakPtr<FConcertSessionActivity> Activity, const FName& ColumnId)
{
	if (TSharedPtr<FConcertSessionActivity> ActivityPin = Activity.Pin())
	{
		if (ActivityView->IsLastColumn(ColumnId) && !ActivityPin->Activity.bIgnored) // The most right cell and activity will not be ignored on recovery.
		{
			// The green 'Recover Through' button that appears in the most right cell if the row is selected.
			return SNew(SBox)
			.Padding(FMargin(1, 1))
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(SPositiveActionButton)
				.Visibility_Lambda([this, Activity](){ return GetRecoverThroughButtonVisibility(Activity.Pin()); })
				.OnClicked_Lambda([this, Activity](){ RecoverThrough(Activity.Pin()); return FReply::Handled(); })
				.ToolTipText(this, &SConcertSessionRecovery::GetRecoverThroughButtonTooltip)
				.Icon(FAppStyle::GetBrush("Icons.CircleArrowRight"))
			];
		}
	}

	return nullptr; // No overlay.
}

void SConcertSessionRecovery::Reset()
{
	ActivityView->ResetActivityList();
}

int32 SConcertSessionRecovery::GetTotalActivityNum() const
{
	return ActivityView->GetTotalActivityNum();
}

TSharedPtr<FConcertSessionActivity> SConcertSessionRecovery::GetMostRecentActivity() const
{
	return ActivityView->GetMostRecentActivity();
}

EVisibility SConcertSessionRecovery::GetRecoverThroughButtonVisibility(TSharedPtr<FConcertSessionActivity> Activity)
{
	return Activity == ActivityView->GetSelectedActivity() && IsRecoverThroughButtonVisible.Get() ? EVisibility::Visible : EVisibility::Hidden;
}

FText SConcertSessionRecovery::GetRecoverThroughButtonTooltip() const
{
	if ((ActivityView->GetTotalActivityNum() == ActivityView->GetDisplayedActivityNum() && ActivityView->GetIgnoredActivityNum() == 0) ||
	    (ActivityView->GetTotalActivityNum() - ActivityView->GetDisplayedActivityNum() == ActivityView->GetIgnoredActivityNum()))
	{
		return LOCTEXT("RecoverThroughAll_Tooltip", "Recover all prior activities through this activity.");
	}
	else if (ActivityView->GetTotalActivityNum() == ActivityView->GetDisplayedActivityNum())
	{
		return LOCTEXT("RecoverThroughWithIgnored_Tooltip", "Recover all prior activities through this activity, excluding unrecoverable ones (grayed out).");
	}
	else if (ActivityView->GetIgnoredActivityNum() == 0 || !ActivityViewOptions->bDisplayIgnoredActivities)
	{
		return LOCTEXT("RecoverThroughWithFilter_Tooltip", "Recover all prior activities through this activity, including the ones currently filtered out by a text search or a view option.");
	}

	return LOCTEXT("RecoverThroughWithIgnoredAndFilter_Tooltip", "Recover all prior activities through this activity, excluding unrecoverable ones (grayed out), but including the those currently filtered out by a text search or a view option.");
}

FText SConcertSessionRecovery::GetRecoverAllButtonTooltip() const
{
	if ((ActivityView->GetTotalActivityNum() == ActivityView->GetDisplayedActivityNum() && ActivityView->GetIgnoredActivityNum() == 0) ||
	    (ActivityView->GetTotalActivityNum() - ActivityView->GetDisplayedActivityNum() == ActivityView->GetIgnoredActivityNum()))
	{
		return LOCTEXT("RecoverAll_Tooltip", "Recover all activities.");
	}
	else if (ActivityView->GetTotalActivityNum() == ActivityView->GetDisplayedActivityNum())
	{
		return LOCTEXT("RecoverAllWithIgnored_Tooltip", "Recover all activities, excluding unrecoverable ones (grayed out).");
	}
	else if (ActivityView->GetIgnoredActivityNum() == 0 || !ActivityViewOptions->bDisplayIgnoredActivities)
	{
		return LOCTEXT("RecoverAllWithFilter_Tooltip", "Recover all activities, including the ones currently filtered out by a text search or a view option.");
	}

	return LOCTEXT("RecoverAllWithIgnoredAndFilter_Tooltip", "Recover all activities, excluding unrecoverable ones (grayed out), but including those currently filtered out by a text search or a view option.");
}

void SConcertSessionRecovery::OnSearchTextChanged(const FText& InFilterText)
{
	SearchText = InFilterText;
	SearchBox->SetError(ActivityView->UpdateTextFilter(InFilterText));
}

void SConcertSessionRecovery::OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type CommitType)
{
	if (!InFilterText.EqualTo(SearchText))
	{
		OnSearchTextChanged(InFilterText);
	}
}

FText SConcertSessionRecovery::HighlightSearchText() const
{
	return SearchText;
}

FReply SConcertSessionRecovery::OnCancelRecoveryClicked()
{
	check(!RecoveryThroughItem.IsValid());

	if (OnCancelFn) // If a cancel behavior is bound.
	{
		OnCancelFn();
	}
	else // Default cancel behavior.
	{
		DismissWindow();
	}
	return FReply::Handled();
}

FReply SConcertSessionRecovery::OnRecoverAllClicked()
{
	// Recover to the most recent activity, ignoring any filter being applied to the view.
	RecoverThrough(ActivityView->GetMostRecentActivity());
	return FReply::Handled();
}

void SConcertSessionRecovery::RecoverThrough(TSharedPtr<FConcertSessionActivity> Item)
{
	bool bShouldDismissWindow = true;
	if (Item)
	{
		RecoveryThroughItem = Item;

		if (OnRestoreFn)
		{
			bShouldDismissWindow = OnRestoreFn(RecoveryThroughItem);
		}
	}

	if (bShouldDismissWindow)
	{
		DismissWindow();
	}
}

void SConcertSessionRecovery::DismissWindow()
{
	if (TSharedPtr<SWindow> ParentWindowPin = ParentWindow.Pin())
	{
		ParentWindowPin->RequestDestroyWindow();
	}
}

#undef LOCTEXT_NAMESPACE
