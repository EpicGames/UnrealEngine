// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPackageDetails.h"

#include "ConcertWorkspaceData.h"
#include "Framework/Views/TableViewMetadata.h"
#include "SlateOptMacros.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SPackageDetails"

namespace PackageDetailsUI
{
	static const FName RevisionLabel(TEXT("Revision"));
	static const FName NameLabel(TEXT("Name"));
	static const FName ModifiedByLabel(TEXT("ModifiedBy"));
	static const FName ActionLabel(TEXT("Action"));
}

class SPackageDetailsRow : public SMultiColumnTableRow<TSharedPtr<int32>>
{
	SLATE_BEGIN_ARGS(SPackageDetailsRow) {}
		SLATE_ARGUMENT(FText, Revision)
		SLATE_ARGUMENT(FText, Name)
		SLATE_ARGUMENT(FText, ModifiedBy)
		SLATE_ARGUMENT(FText, Action)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		Revision = InArgs._Revision;
		Name = InArgs._Name;
		ModifiedBy = InArgs._ModifiedBy;
		Action = InArgs._Action;

		SMultiColumnTableRow<TSharedPtr<int32> >::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

public:

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		FText ColumnText;

		if (ColumnName == PackageDetailsUI::RevisionLabel)
		{
			ColumnText = Revision;
		}
		else if (ColumnName == PackageDetailsUI::NameLabel)
		{
			ColumnText = Name;
		}
		else if (ColumnName == PackageDetailsUI::ModifiedByLabel)
		{
			ColumnText = ModifiedBy;
		}
		else if (ColumnName == PackageDetailsUI::ActionLabel)
		{
			ColumnText = Action;
		}

		return SNew(SBox)
			.Padding(FMargin(4.0, 0.0))
			[
				SNew(STextBlock)
				.Text(ColumnText)
			];
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

private:

	/** Package revision. */
	FText Revision;

	/** Package name. */
	FText Name;

	/** Name of the user who modified the package. */
	FText ModifiedBy;

	/** What the modification was. */
	FText Action;
};

void SPackageDetails::Construct(const FArguments& InArgs)
{
	ChildSlot
		[
			SAssignNew(DetailListView, SListView<FListItemPtr>)
			.ListItemsSource(&DetailItems)
			.OnGenerateRow(this, &SPackageDetails::HandleGenerateRow)
			.HeaderRow
			(
				SNew(SHeaderRow)
				+SHeaderRow::Column(PackageDetailsUI::RevisionLabel)
				.DefaultLabel(LOCTEXT("RevisionFlagsHeaderName", "Revision"))
				+SHeaderRow::Column(PackageDetailsUI::NameLabel)
				.DefaultLabel(LOCTEXT("NameColumnHeaderName", "Name"))
				+SHeaderRow::Column(PackageDetailsUI::ModifiedByLabel)
				.DefaultLabel(LOCTEXT("ModifiedByColumnHeaderName", "ModifiedBy"))
				+SHeaderRow::Column(PackageDetailsUI::ActionLabel)
				.DefaultLabel(LOCTEXT("ActionColumnHeaderName", "Action"))
			)
		];
}

void SPackageDetails::SetPackageInfo(const FConcertPackageInfo& InPackageInfo, const int64 InRevision, const FString& InModifiedBy)
{
	DetailItems.Reset();
	
	FText Action = LOCTEXT("InvalidPackageAction", "Invalid Action");

	switch (InPackageInfo.PackageUpdateType)
	{
		case EConcertPackageUpdateType::Added:
			Action = LOCTEXT("PackageActionAdd", "Add");
			break;
		case EConcertPackageUpdateType::Deleted:
			Action = LOCTEXT("PackageActionDelete", "Delete");
			break;
		case EConcertPackageUpdateType::Renamed:
			Action = LOCTEXT("PackageActionRename", "Rename/Move");
			break;
		case EConcertPackageUpdateType::Saved:
			Action = LOCTEXT("PackageActionSave", "Save");
			break;
		default:
			break;
	}

	FPackageDetailsRow Row
	{
		FText::FromName(InPackageInfo.PackageName),
		FText::FromString(LexToString(InRevision)),
		InModifiedBy.IsEmpty() ? LOCTEXT("UnknownUser", "Unknown") : FText::FromString(InModifiedBy),
		MoveTemp(Action)
	};

	DetailItems.Emplace(MakeShared<FPackageDetailsRow>(MoveTemp(Row)));

	if (InPackageInfo.NewPackageName != FName())
	{
		FPackageDetailsRow RenamedRow;
		RenamedRow.PackageName = FText::FromString(TEXT("-> ") + InPackageInfo.NewPackageName.ToString());
		DetailItems.Emplace(MakeShared<FPackageDetailsRow>(MoveTemp(RenamedRow)));
	}

	DetailListView->RequestListRefresh();
}

TSharedRef<ITableRow> SPackageDetails::HandleGenerateRow(FListItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable) const
{	
	return SNew(SPackageDetailsRow, OwnerTable)
		.Revision(InItem->Revision)
		.Name(InItem->PackageName)
		.ModifiedBy(InItem->ModifiedBy)
		.Action(InItem->Action);
}

#undef LOCTEXT_NAMESPACE /** SPackageDetails */
