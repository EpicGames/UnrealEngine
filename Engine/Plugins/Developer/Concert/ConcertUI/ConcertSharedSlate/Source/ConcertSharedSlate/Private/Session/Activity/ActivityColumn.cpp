// Copyright Epic Games, Inc. All Rights Reserved.

#include "Session/Activity/ActivityColumn.h"

void FActivityColumn::BuildColumnWidget(const TSharedRef<SConcertSessionActivities>& Owner,  const TSharedRef<FConcertSessionActivity>& Activity, SOverlay::FScopedWidgetSlotArguments& Slot) const
{
    if (GenerateColumnWidgetCallback.IsBound())
    {
	    GenerateColumnWidgetCallback.Execute(Owner, Activity, Slot);
    }
}

void FActivityColumn::ExecutePopulateSearchString(const TSharedRef<SConcertSessionActivities>& Owner, const FConcertSessionActivity& Activity, TArray<FString>& ExistingSearchStrings) const
{
	if (PopulateSearchStringCallback.IsBound())
	{
		PopulateSearchStringCallback.Execute(Owner, Activity, ExistingSearchStrings);
	}
}
