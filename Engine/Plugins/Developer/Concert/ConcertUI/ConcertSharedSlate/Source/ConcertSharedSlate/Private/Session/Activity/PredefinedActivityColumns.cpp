// Copyright Epic Games, Inc. All Rights Reserved.

#include "Session/Activity/PredefinedActivityColumns.h"

#include "ConcertSyncSessionTypes.h"
#include "Session/Activity/SConcertSessionActivities.h"
#include "SessionActivityUtils.h"

#include "Session/Activity/ActivityColumn.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Text/SRichTextBlock.h"

#define LOCTEXT_NAMESPACE "SConcertSessionActivities"

namespace UE::ConcertSharedSlate::Private
{
	static FText GetActivityDateTime(const FConcertSessionActivity& Activity, ETimeFormat TimeFormat)
	{
		return TimeFormat == ETimeFormat::Relative ? ConcertFrontendUtils::FormatRelativeTime(Activity.Activity.EventTime) : FText::AsDateTime(Activity.Activity.EventTime);
	}
}

const FName UE::ConcertSharedSlate::ActivityColumn::DateTimeColumnId	= TEXT("DateTime");
const FName UE::ConcertSharedSlate::ActivityColumn::SummaryColumnId     = TEXT("Summary");
const FName UE::ConcertSharedSlate::ActivityColumn::AvatarColorColumnId = TEXT("Client AvatarColor");
const FName UE::ConcertSharedSlate::ActivityColumn::ClientNameColumnId  = TEXT("Client");
const FName UE::ConcertSharedSlate::ActivityColumn::OperationColumnId   = TEXT("Operation");
const FName UE::ConcertSharedSlate::ActivityColumn::PackageColumnId     = TEXT("Package");

FActivityColumn UE::ConcertSharedSlate::ActivityColumn::DateTime()
{
	return FActivityColumn(
		SHeaderRow::Column(DateTimeColumnId)
			.DefaultLabel(LOCTEXT("DateTime", "Date/Time"))
			.FillSized(160)
		)
		.ColumnSortOrder(static_cast<int32>(EPredefinedColumnOrder::DateTime))
		.GenerateColumnWidget(FActivityColumn::FGenerateColumnWidget::CreateLambda([](const TSharedRef<SConcertSessionActivities>& Owner, const TSharedRef<FConcertSessionActivity>& Activity, SOverlay::FScopedWidgetSlotArguments& Slot)
		{
			Slot
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text_Lambda([TimeFormat = Owner->GetTimeFormat(), WeakActivity = TWeakPtr<FConcertSessionActivity>(Activity)]()
				{
					if (const TSharedPtr<FConcertSessionActivity> ItemPin = WeakActivity.Pin())
					{
						return Private::GetActivityDateTime(*ItemPin, TimeFormat.Get());
					}
					return FText::GetEmpty();
				})
				.HighlightText(Owner->GetHighlightText())
			];
		}))
		.PopulateSearchString(FActivityColumn::FPopulateSearchString::CreateLambda([](const TSharedRef<SConcertSessionActivities>& Owner, const FConcertSessionActivity& Activity, TArray<FString>& OutSearchStrings)
		{
			OutSearchStrings.Add(Private::GetActivityDateTime(Activity, Owner->GetTimeFormat().Get()).ToString());
		}));
}

FActivityColumn UE::ConcertSharedSlate::ActivityColumn::Summary()
{
	return FActivityColumn(
		SHeaderRow::Column(SummaryColumnId)
			.DefaultLabel(LOCTEXT("Summary", "Summary"))
			.ShouldGenerateWidget(true)
		)
		.GenerateColumnWidget(FActivityColumn::FGenerateColumnWidget::CreateLambda([](const TSharedRef<SConcertSessionActivities>& Owner, const TSharedRef<FConcertSessionActivity>& Activity, SOverlay::FScopedWidgetSlotArguments& Slot)
		{
			const FConcertSyncActivitySummary* Summary = Activity->ActivitySummary.Cast<FConcertSyncActivitySummary>();
			constexpr bool bRichText = true;
			const FText SummaryText = Summary ? Summary->ToDisplayText(FText::GetEmpty(), bRichText) : FText::GetEmpty();
			Slot
			.VAlign(VAlign_Center)
			[
				SNew(SRichTextBlock)
				.DecoratorStyleSet(FConcertFrontendStyle::Get().Get())
				.Text(SummaryText)
				.HighlightText(Owner->GetHighlightText())
			];
		}))
		.PopulateSearchString(FActivityColumn::FPopulateSearchString::CreateLambda([](const TSharedRef<SConcertSessionActivities>& Owner, const FConcertSessionActivity& Activity, TArray<FString>& OutSearchStrings)
		{
			if (const FConcertSyncActivitySummary* Summary = Activity.ActivitySummary.Cast<FConcertSyncActivitySummary>();
				Summary)
			{
				const TOptional<FConcertClientInfo> ClientInfo = Owner->GetClientInfo(Activity.Activity.EndpointId);
				const FText ClientName = ClientInfo ? FText::AsCultureInvariant(ClientInfo->DisplayName) : FText::GetEmpty();

				constexpr bool bRichText = false;
				OutSearchStrings.Add(Summary->ToDisplayText(ClientName, bRichText).ToString());
			}
		}));
}

FActivityColumn UE::ConcertSharedSlate::ActivityColumn::AvatarColor()
{
	return FActivityColumn(
		SHeaderRow::Column(AvatarColorColumnId)
			.DefaultLabel(INVTEXT(""))
			.FixedWidth(8)
			.ShouldGenerateWidget(true)
		)
		.ColumnSortOrder(static_cast<int32>(EPredefinedColumnOrder::AvatarColor))
		.GenerateColumnWidget(FActivityColumn::FGenerateColumnWidget::CreateLambda([](const TSharedRef<SConcertSessionActivities>& Owner, const TSharedRef<FConcertSessionActivity>& Activity, SOverlay::FScopedWidgetSlotArguments& Slot)
		{
			const TOptional<FConcertClientInfo> ClientInfo = Owner->GetClientInfo(Activity->Activity.EndpointId);
			const FLinearColor AvatarColor = ClientInfo ? ClientInfo->AvatarColor : FConcertFrontendStyle::Get()->GetColor("Concert.DisconnectedColor");
			Slot
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(2, 1)
			[
				SNew(SColorBlock)
				.Color(AvatarColor)
				.Size(FVector2D(4.f, 16.f))
			];
		}));
}

FActivityColumn UE::ConcertSharedSlate::ActivityColumn::ClientName()
{
	return FActivityColumn(
		SHeaderRow::Column(ClientNameColumnId)
			.DefaultLabel(LOCTEXT("Client", "Client"))
			.FillSized(80)
		)
		.ColumnSortOrder(static_cast<int32>(EPredefinedColumnOrder::ClientName))
		.GenerateColumnWidget(FActivityColumn::FGenerateColumnWidget::CreateLambda([](const TSharedRef<SConcertSessionActivities>& Owner, const TSharedRef<FConcertSessionActivity>& Activity, SOverlay::FScopedWidgetSlotArguments& Slot)
		{
			const TOptional<FConcertClientInfo> ClientInfo = Owner->GetClientInfo(Activity.Get().Activity.EndpointId);
			const FText ClientName = ClientInfo ? FText::AsCultureInvariant(ClientInfo->DisplayName) : FText::GetEmpty();
			
			Slot
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(ClientName)
				.HighlightText(Owner->GetHighlightText())
			];
		}))
		.PopulateSearchString(FActivityColumn::FPopulateSearchString::CreateLambda([](const TSharedRef<SConcertSessionActivities>& Owner, const FConcertSessionActivity& Activity, TArray<FString>& OutSearchStrings)
		{
			if (const TOptional<FConcertClientInfo> ClientInfo = Owner->GetClientInfo(Activity.Activity.EndpointId))
			{
				const FText ClientName = ClientInfo ? FText::AsCultureInvariant(ClientInfo->DisplayName) : FText::GetEmpty();
				OutSearchStrings.Add(ClientName.ToString());
			}
		}));
}

FActivityColumn UE::ConcertSharedSlate::ActivityColumn::Operation()
{
	return FActivityColumn(
		SHeaderRow::Column(OperationColumnId)
			.DefaultLabel(LOCTEXT("Operation", "Operation"))
			.FillSized(160)
		)
		.ColumnSortOrder(static_cast<int32>(EPredefinedColumnOrder::Operation))
		.GenerateColumnWidget(FActivityColumn::FGenerateColumnWidget::CreateLambda([](const TSharedRef<SConcertSessionActivities>& Owner, const TSharedRef<FConcertSessionActivity>& Activity, SOverlay::FScopedWidgetSlotArguments& Slot)
		{
			Slot
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(Private::GetOperationName(*Activity))
				.HighlightText(Owner->GetHighlightText())
			];
		}))
		.PopulateSearchString(FActivityColumn::FPopulateSearchString::CreateLambda([](const TSharedRef<SConcertSessionActivities>& Owner, const FConcertSessionActivity& Activity, TArray<FString>& OutSearchStrings)
		{
			OutSearchStrings.Add(Private::GetOperationName(Activity).ToString());
		}));
}

FActivityColumn UE::ConcertSharedSlate::ActivityColumn::Package()
{
	return FActivityColumn(
		SHeaderRow::Column(PackageColumnId)
			.DefaultLabel(LOCTEXT("Package", "Package"))
			.FillSized(200)
		)
		.ColumnSortOrder(static_cast<int32>(EPredefinedColumnOrder::Package))
		.GenerateColumnWidget(FActivityColumn::FGenerateColumnWidget::CreateLambda([](const TSharedRef<SConcertSessionActivities>& Owner, const TSharedRef<FConcertSessionActivity>& Activity, SOverlay::FScopedWidgetSlotArguments& Slot)
		{
			Slot
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(Private::GetPackageName(*Activity))
				.HighlightText(Owner->GetHighlightText())
			];
		}))
		.PopulateSearchString(FActivityColumn::FPopulateSearchString::CreateLambda([](const TSharedRef<SConcertSessionActivities>& Owner, const FConcertSessionActivity& Activity, TArray<FString>& OutSearchStrings)
		{
			OutSearchStrings.Add(Private::GetPackageName(Activity).ToString());
		}));
}

#undef LOCTEXT_NAMESPACE
