// Copyright Epic Games, Inc. All Rights Reserved.

#include "Session/Activity/SConcertSessionActivities.h"

#include "Async/Future.h"
#include "ConcertLogGlobal.h"
#include "ConcertSyncSessionTypes.h"
#include "Session/Activity/IConcertReflectionDataProvider.h"
#include "Framework/Views/TableViewMetadata.h"
#include "SessionActivityUtils.h"
#include "Misc/TextFilter.h"
#include "SPackageDetails.h"

#include "Misc/AsyncTaskNotification.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Replication/Messages/ReplicationActivity.h"
#include "Session/Activity/ActivityColumn.h"
#include "Widgets/SUndoHistoryDetails.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Layout/SScrollBox.h"

#include "Session/Activity/PredefinedActivityColumns.h"

#define LOCTEXT_NAMESPACE "SConcertSessionActivities"

namespace ConcertSessionActivityUtils
{
// The View Options check boxes.
const FName DisplayRelativeTimeCheckBoxId       = TEXT("DisplayRelativeTime");
const FName ShowConnectionActivitiesCheckBoxId  = TEXT("ShowConnectionActivities");
const FName ShowLockActivitiesCheckBoxId        = TEXT("ShowLockActivities");
const FName ShowPackageActivitiesCheckBoxId     = TEXT("ShowPackageActivities");
const FName ShowTransactionActivitiesCheckBoxId = TEXT("ShowTransactionActivities");
const FName ShowReplicationActivitiesCheckBoxId = TEXT("ShowReplicationActivities");
const FName ShowIgnoredActivitiesCheckBoxId     = TEXT("ShowIgnoredActivities");
	
FText GetSummary(const FConcertSessionActivity& Activity, const FText& ClientName, bool bAsRichText)
{
	if (const FConcertSyncActivitySummary* Summary = Activity.ActivitySummary.Cast<FConcertSyncActivitySummary>())
	{
		return Summary->ToDisplayText(ClientName, bAsRichText);
	}

	return FText::GetEmpty();
}

FText GetClientName(const TOptional<FConcertClientInfo>& InActivityClient)
{
	return InActivityClient ? FText::AsCultureInvariant(InActivityClient->DisplayName) : FText::GetEmpty();
}
	
FText GetClientName(const FConcertClientInfo* InActivityClient)
{
	return InActivityClient ? FText::AsCultureInvariant(InActivityClient->DisplayName) : FText::GetEmpty();
}

};

/**
 * Displays the summary of an activity recorded and recoverable in the SConcertSessionRecovery list view.
 */
class SConcertSessionActivityRow : public SMultiColumnTableRow<TSharedPtr<FConcertSessionActivity>>
{
public:

	DECLARE_DELEGATE_RetVal_OneParam(const FActivityColumn*, FGetColumn,
		const FName& ColumnId
		);
	
	SLATE_BEGIN_ARGS(SConcertSessionActivityRow)
		: _OnMakeColumnOverlayWidget()
		, _DarkenMutedActivities(true)
	{}
		/** Function invoked when generating a row to add a widget above the column widget. */
		SLATE_ARGUMENT(SConcertSessionActivities::FMakeColumnOverlayWidgetFunc, OnMakeColumnOverlayWidget)
		/** Optional. Whether to reduce focus to activities by darkening when the activity is muted (default: true). */
		SLATE_ARGUMENT(bool, DarkenMutedActivities) 
	SLATE_END_ARGS()

	/**
	 * Constructs a row widget to display a Concert activity.
	 * @param InArgs The widgets arguments.
	 * @param InActivity The activity to display.
	 * @param InActivityClient The client who produced this activity. Can be null if unknown or not desirable.
	 * @param InOwnerTableView The table view that will own this row.
	 */
	void Construct(const FArguments& InArgs, TSharedRef<SConcertSessionActivities> InOwner, TSharedRef<FConcertSessionActivity> InActivity, FGetColumn InColumnGetter, const TOptional<FConcertClientInfo>& InActivityClient, const TSharedRef<STableViewBase>& InOwnerTableView);

	/** Generates the widget representing this row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:
	
	/** Returns the display name of the client who performed the activity or an empty text if the information wasn't available or desired. */
	FText GetClientName() const { return ClientName; }

	/** Generates the tooltip for this row. */
	FText MakeTooltipText() const;

private:
	TWeakPtr<SConcertSessionActivities> Owner;
	TWeakPtr<FConcertSessionActivity> Activity;
	FGetColumn ColumnGetter;
	FText AbsoluteDateTime;
	FText ClientName;
	SConcertSessionActivities::FMakeColumnOverlayWidgetFunc OnMakeColumnOverlayWidget;
	bool bDarkenMutedActivities;
};


void SConcertSessionActivityRow::Construct(const FArguments& InArgs, TSharedRef<SConcertSessionActivities> InOwner, TSharedRef<FConcertSessionActivity> InActivity, FGetColumn InColumnGetter, const TOptional<FConcertClientInfo>& InActivityClient, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	check(InColumnGetter.IsBound());
	Owner = InOwner;
	Activity = InActivity;
	ColumnGetter = InColumnGetter;
	
	OnMakeColumnOverlayWidget = InArgs._OnMakeColumnOverlayWidget;
	AbsoluteDateTime = ConcertFrontendUtils::FormatTime(InActivity->Activity.EventTime, ETimeFormat::Absolute); // Cache the absolute time. It doesn't changes.
	ClientName = ConcertSessionActivityUtils::GetClientName(InActivityClient);
	bDarkenMutedActivities = InArgs._DarkenMutedActivities;
	
	// Construct base class
	SMultiColumnTableRow<TSharedPtr<FConcertSessionActivity>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);

	const bool bDarkenMutedActivity = bDarkenMutedActivities && (InActivity->Activity.Flags & EConcertSyncActivityFlags::Muted) != EConcertSyncActivityFlags::None;
	if (InActivity->Activity.bIgnored || bDarkenMutedActivity)
	{
		SetColorAndOpacity(FLinearColor(0.5, 0.5, 0.5, 0.5));
	}
}

TSharedRef<SWidget> SConcertSessionActivityRow::GenerateWidgetForColumn(const FName& ColumnId)
{
	const TSharedPtr<SConcertSessionActivities> OwnerPin = Owner.Pin();
	const TSharedPtr<FConcertSessionActivity> ActivityPin = Activity.Pin();
	check(OwnerPin && ActivityPin);
	
	const TSharedRef<SOverlay> Overlay = SNew(SOverlay);
	if (const FActivityColumn* Column = ColumnGetter.Execute(ColumnId))
	{
		SOverlay::FScopedWidgetSlotArguments WidgetSlot = Overlay->AddSlot();
		Column->BuildColumnWidget(OwnerPin.ToSharedRef(), ActivityPin.ToSharedRef(), WidgetSlot);
	}
	
	if (OnMakeColumnOverlayWidget.IsBound())
	{
		if (TSharedPtr<SWidget> OverlayedWidget = OnMakeColumnOverlayWidget.Execute(SharedThis(this), ActivityPin, ColumnId))
		{
			Overlay->AddSlot()
			[
				OverlayedWidget.ToSharedRef()
			];
		}
	}

	SetToolTipText(TAttribute<FText>(this, &SConcertSessionActivityRow::MakeTooltipText));
	return Overlay;
}

FText SConcertSessionActivityRow::MakeTooltipText() const
{
	if (TSharedPtr<FConcertSessionActivity> ActivityPin = Activity.Pin())
	{
		const FText Client = GetClientName();
		const FText Operation = UE::ConcertSharedSlate::Private::GetOperationName(*ActivityPin);
		const FText Package = UE::ConcertSharedSlate::Private::GetPackageName(*ActivityPin);
		const FText Summary = ConcertSessionActivityUtils::GetSummary(*ActivityPin, Client, /*bAsRichText*/false);

		FTextBuilder TextBuilder;

		if (!Operation.IsEmpty())
		{
			TextBuilder.AppendLine(Operation);
		}

		TextBuilder.AppendLineFormat(LOCTEXT("ActivityRowTooltip_DateTime", "{0} ({1})"), AbsoluteDateTime, ConcertFrontendUtils::FormatRelativeTime(ActivityPin->Activity.EventTime));

		if (!Package.IsEmpty())
		{
			TextBuilder.AppendLine(Package);
		}

		if (!Summary.IsEmpty())
		{
			TextBuilder.AppendLine(Summary);
		}

		if (ActivityPin->Activity.bIgnored)
		{
			TextBuilder.AppendLine();
			TextBuilder.AppendLine(LOCTEXT("IgnoredActivity", "** This activity cannot be recovered (likely recorded during a Multi-User session). It is displayed for crash inspection only. It will be ignored on restore."));
		}
		
		const bool bDarkenMutedActivity = bDarkenMutedActivities && (ActivityPin->Activity.Flags & EConcertSyncActivityFlags::Muted) != EConcertSyncActivityFlags::None;
		if (bDarkenMutedActivity)
		{
			TextBuilder.AppendLine(LOCTEXT("MutedActivity", "** This activity is muted."));
		}

		return TextBuilder.ToText();
	}

	return FText::GetEmpty();
}

void SConcertSessionActivities::Construct(const FArguments& InArgs)
{
	FetchActivitiesFn = InArgs._OnFetchActivities;
	GetActivityUserFn = InArgs._OnMapActivityToClient;
	GetTransactionEventFn = InArgs._OnGetTransactionEvent;
	GetPackageEventFn = InArgs._OnGetPackageEvent;
	MakeColumnOverlayWidgetFn = InArgs._OnMakeColumnOverlayWidget;
	OnContextMenuOpening = InArgs._OnContextMenuOpening;
	
	HighlightText = InArgs._HighlightText;
	
	TimeFormat = InArgs._TimeFormat;
	ConnectionActivitiesVisibility = InArgs._ConnectionActivitiesVisibility;
	LockActivitiesVisibility = InArgs._LockActivitiesVisibility;
	PackageActivitiesVisibility = InArgs._PackageActivitiesVisibility;
	TransactionActivitiesVisibility = InArgs._TransactionActivitiesVisibility;
	ReplicationActivitiesVisibility = InArgs._ReplicationActivitiesVisibility;
	IgnoredActivitiesVisibility = InArgs._IgnoredActivitiesVisibility;
	DetailsAreaVisibility = InArgs._DetailsAreaVisibility;
	bAutoScrollDesired = InArgs._IsAutoScrollEnabled;
	bDarkenMutedActivities = InArgs._DarkenMutedActivities;

	SearchTextFilter = MakeShared<TTextFilter<const FConcertSessionActivity&>>(TTextFilter<const FConcertSessionActivity&>::FItemToStringArray::CreateSP(this, &SConcertSessionActivities::PopulateSearchStrings));
	SearchTextFilter->OnChanged().AddSP(this, &SConcertSessionActivities::OnActivityFilterUpdated);

	if (InArgs._UndoHistoryReflectionProvider)
	{
		ConcertReflectionDataProvider = InArgs._UndoHistoryReflectionProvider;
	}
	const TSharedRef<UE::UndoHistory::IReflectionDataProvider> ReflectionDataProvider =
		ConcertReflectionDataProvider ? ConcertReflectionDataProvider->ToSharedRef() : UE::UndoHistory::CreateDefaultReflectionProvider();

	CreateHeaderRow(InArgs);
	ActiveFilterFlags = QueryActiveActivityFilters();
	ChildSlot
	[
		SNew(SSplitter)
		.Orientation(Orient_Vertical)

		// Activity List
		+SSplitter::Slot()
		.Value(0.75)
		[
			SNew(SOverlay)

			+SOverlay::Slot() // Activity list itself.
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
				.BorderBackgroundColor(FSlateColor(FLinearColor(0.6f, 0.6f, 0.6f)))
				.Padding(0)
				[
					SAssignNew(ActivityView, SListView<TSharedPtr<FConcertSessionActivity>>)
					.ListItemsSource(&Activities)
					.OnGenerateRow(this, &SConcertSessionActivities::OnGenerateActivityRowWidget)
					.SelectionMode(InArgs._SelectionMode.Get(ESelectionMode::Single))
					.AllowOverscroll(EAllowOverscroll::No)
					.OnListViewScrolled(this, &SConcertSessionActivities::OnListViewScrolled)
					.OnSelectionChanged(this, &SConcertSessionActivities::OnListViewSelectionChanged)
					.HeaderRow(HeaderRow)
					.OnContextMenuOpening_Lambda([this]() -> TSharedPtr<SWidget>
					{
						if (ActivityView->GetSelectedItems().Num() == 0)
						{
							return UE::ConcertSharedSlate::MakeTableContextMenu(HeaderRow.ToSharedRef(), {}, true);
						}

						if (OnContextMenuOpening.IsBound())
						{
							return OnContextMenuOpening.Execute();
						}
						return {};
					})
				]
			]

			+SOverlay::Slot() // Display a reason why no activities are shown.
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Visibility_Lambda([TextAttr = InArgs._NoActivitiesReasonText](){ return TextAttr.Get().IsEmptyOrWhitespace() ? EVisibility::Collapsed : EVisibility::Visible; })
				.Text(InArgs._NoActivitiesReasonText)
				.Justification(ETextJustify::Center)
			]
		]

		// Activity details.
		+SSplitter::Slot()
		.Value(0.25)
		.SizeRule(TAttribute<SSplitter::ESizeRule>(this, &SConcertSessionActivities::GetDetailsAreaSizeRule))
		[
			SAssignNew(ExpandableDetails, SExpandableArea)
			.Visibility(GetDetailAreaVisibility())
			.InitiallyCollapsed(true)
			.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
			.BorderImage_Lambda([this]() { return ConcertFrontendUtils::GetExpandableAreaBorderImage(*ExpandableDetails); })
			.BodyBorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			.BodyBorderBackgroundColor(FLinearColor::White)
			.OnAreaExpansionChanged(this, &SConcertSessionActivities::OnDetailsAreaExpansionChanged)
			.Padding(0.0f)
			.HeaderContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Details", "Details"))
				.Font(FAppStyle::Get().GetFontStyle("DetailsView.CategoryFontStyle"))
				.ShadowOffset(FVector2D(1.0f, 1.0f))
			]
			.BodyContent()
			[
				SNew(SOverlay)

				+SOverlay::Slot()
				[
					SNew(SScrollBox)
					.ScrollBarThickness(FVector2D(12.0f, 5.0f)) // To have same thickness than the ListView scroll bar.

					+SScrollBox::Slot()
					[
						SAssignNew(TransactionDetailsPanel, SUndoHistoryDetails, ReflectionDataProvider)
						.Visibility(EVisibility::Collapsed)
					]

					+SScrollBox::Slot()
					[
						SAssignNew(PackageDetailsPanel, SPackageDetails)
						.Visibility(EVisibility::Collapsed)
					]
				]

				+SOverlay::Slot()
				[
					SAssignNew(NoDetailsPanel, SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Visibility(EVisibility::Visible)
					[
						SNew(STextBlock)
						.Text(this, &SConcertSessionActivities::GetNoDetailsText)
					]
				]

				+SOverlay::Slot()
				[
					SAssignNew(LoadingDetailsPanel, SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Visibility(EVisibility::Collapsed)
					[
						SNew(SThrobber)
					]
				]
			]
		]
	];

	// Check if some activities are already available.
	FetchActivities();

	if (bAutoScrollDesired)
	{
		FSlateApplication::Get().OnPostTick().AddSP(this, &SConcertSessionActivities::OnPostTick);
	}
}

TSharedRef<SHeaderRow> SConcertSessionActivities::CreateHeaderRow(const FArguments& InArgs)
{
	using namespace UE::ConcertSharedSlate;
	TArray<FActivityColumn> Columns = InArgs._Columns;
	
	// Before sort because columns can be placed before
	const bool bContainsDateTime = Columns.FindByPredicate([](const FActivityColumn& Column) { return Column.ColumnId ==  ActivityColumn::DateTimeColumnId; }) != nullptr; 
	checkf(!bContainsDateTime, TEXT("DateTime is already created by SConcertSessionActivities!"))
	Columns.Add(ActivityColumn::DateTime());
	
	Columns.Sort([](const FActivityColumn& Left, const FActivityColumn& Right) { return Left.GetColumnSortOrderValue() < Right.GetColumnSortOrderValue(); });

	// Summary is always last
	const bool bContainsSummary = Columns.FindByPredicate([](const FActivityColumn& Column) { return Column.ColumnId ==  ActivityColumn::SummaryColumnId; }) != nullptr; 
	checkf(!bContainsSummary, TEXT("Summary is already created by SConcertSessionActivities!"))
	Columns.Add(ActivityColumn::Summary());
	
	HeaderRow = SNew(SHeaderRow)
		.OnHiddenColumnsListChanged_Lambda([this, SaveCallback = InArgs._SaveColumnVisibilitySnapshot]()
		{
			// bIsUpdatingColumnVisibility is true when the columns visibility is being changed by OnColumnVisibilitySettingsChanged to prevent recursion
			if (SaveCallback.IsBound() && !bIsUpdatingColumnVisibility)
			{
				const FColumnVisibilitySnapshot Snapshot = SnapshotColumnVisibilityState(HeaderRow.ToSharedRef());
				SaveCallback.Execute(Snapshot);
			}
		});
	
	TSet<FName> DuplicateColumnDetection;
	for (FActivityColumn& Column : Columns)
	{
		check(!DuplicateColumnDetection.Contains(Column.ColumnId));
		DuplicateColumnDetection.Add(Column.ColumnId);
		
		// SHeaderRow owns the columns and deletes them when destroyed
		FActivityColumn* ManagedByHeaderRow = new FActivityColumn(MoveTemp(Column));
		HeaderRow->AddColumn(*ManagedByHeaderRow);
	}

	TGuardValue<bool> GuardValue(bIsUpdatingColumnVisibility, true);
	RestoreColumnVisibilityState(HeaderRow.ToSharedRef(), InArgs._ColumnVisibilitySnapshot);
	return HeaderRow.ToSharedRef();
}

TSharedRef<ITableRow> SConcertSessionActivities::OnGenerateActivityRowWidget(TSharedPtr<FConcertSessionActivity> Activity, const TSharedRef<STableViewBase>& OwnerTable)
{
	const TOptional<FConcertClientInfo> ActivityClient = GetActivityUserFn.IsBound() ? GetActivityUserFn.Execute(Activity->Activity.EndpointId) : TOptional<FConcertClientInfo>{};
	const SConcertSessionActivityRow::FGetColumn ColumnGetter = SConcertSessionActivityRow::FGetColumn::CreateLambda([this](const FName& ColumnId) -> const FActivityColumn*
	{
		for (const SHeaderRow::FColumn& Column : HeaderRow->GetColumns())
		{
			if (Column.ColumnId == ColumnId)
			{
				return static_cast<const FActivityColumn*>(&Column);
			}
		}

		checkNoEntry();
		return nullptr;
	});
	
	return SNew(SConcertSessionActivityRow, SharedThis(this), Activity.ToSharedRef(), ColumnGetter, ActivityClient, OwnerTable)
		.OnMakeColumnOverlayWidget(MakeColumnOverlayWidgetFn)
		.DarkenMutedActivities(bDarkenMutedActivities);
}

void SConcertSessionActivities::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	EConcertActivityFilterFlags LatestFilterFlags = QueryActiveActivityFilters();
	if (ActiveFilterFlags != LatestFilterFlags)
	{
		ActiveFilterFlags = LatestFilterFlags;
		OnActivityFilterUpdated();
	}

	FetchActivities(); // Check if we should fetch more activities in case we filtered out too many of them.
}

FText SConcertSessionActivities::GetNoDetailsText() const
{
	return GetSelectedActivity() ?
		LOCTEXT("NoDetails_NotAvailable", "The selected activity doesn't have details to display.") :
		LOCTEXT("NoDetails_NoActivitySelected", "Select an activity to view its details.");
}

void SConcertSessionActivities::OnPostTick(float)
{
	// NOTE: The way the list view adjust the scroll position when the component is resized has some behaviors to consider to get auto-scrolling
	//       working consistently. When the list view shrink (allowing less item) the scroll view doesn't remain anchored at the end. Instead
	//       the scroll position moves a little bit up and the list view doesn't consider it as scrolling because OnListViewScrolled() is not
	//       called. The code below detects that case and maintain the scroll position at the end when required.

	if (bActivityViewScrolled) // OnListViewScrolled() was invoked. The user scrolled the activity list or enlarged the view to see more items.
	{
		bUserScrolling = !ActivityView->GetScrollDistanceRemaining().IsNearlyZero();
		bActivityViewScrolled = false;
	}
	else if (bAutoScrollDesired && !bUserScrolling && ActivityView->GetScrollDistanceRemaining().Y > 0) // See NOTE.
	{
		ActivityView->ScrollToBottom(); // Ensure the scroll position is maintained at the end.
	}
}

void SConcertSessionActivities::OnListViewScrolled(double InScrollOffset)
{
	bActivityViewScrolled = true;

	if (FetchActivitiesFn.IsBound()) // This widget is responsible to populate the view.
	{
		if (!bAllActivitiesFetched && ActivityView->GetScrollDistance().Y > 0.7) // Should fetch more?
		{
			DesiredActivitiesCount += ActivitiesPerRequest; // This will request another 'page' the next time FetchActivities() is called.
		}
	}
}

void SConcertSessionActivities::OnListViewSelectionChanged(TSharedPtr<FConcertSessionActivity> InActivity, ESelectInfo::Type SelectInfo)
{
	UpdateDetailArea(InActivity);
}

void SConcertSessionActivities::OnDetailsAreaExpansionChanged(bool bExpanded)
{
	bDetailsAreaExpanded = bExpanded;
	UpdateDetailArea(bDetailsAreaExpanded ? GetSelectedActivity() : nullptr);
}

void SConcertSessionActivities::UpdateDetailArea(TSharedPtr<FConcertSessionActivity> InSelectedActivity)
{
	if (DetailsAreaVisibility != EVisibility::Visible || !bDetailsAreaExpanded)
	{
		return;
	}
	else if (!InSelectedActivity.IsValid()) // The selection was cleared?
	{
		SetDetailsPanelVisibility(NoDetailsPanel.Get());
	}
	else if (InSelectedActivity->EventPayload) // The event payload is already bundled in the activity stream?
	{
		UpdateDetailsByEventPayload(InSelectedActivity);
	}
	else
	{
		UpdateDetailsByRequest(InSelectedActivity);
	}
}

void SConcertSessionActivities::UpdateDetailsByEventPayload(TSharedPtr<FConcertSessionActivity> InSelectedActivity)
{
	static_assert(static_cast<uint8>(EConcertSyncActivityEventType::Count) == 6, "If you added an EConcertSyncActivityEventType entry, you may want to update this function");
	if (InSelectedActivity->Activity.EventType == EConcertSyncActivityEventType::Transaction)
	{
		FConcertSyncTransactionEvent TransactionEvent;
		InSelectedActivity->EventPayload->GetTypedPayload(TransactionEvent);
		if (TransactionEvent.Transaction.ExportedObjects.Num())
		{
			DisplayTransactionDetails(*InSelectedActivity, TransactionEvent.Transaction);
		}
		else
		{
			SetDetailsPanelVisibility(NoDetailsPanel.Get());
		}
	}
	else if (InSelectedActivity->Activity.EventType == EConcertSyncActivityEventType::Package)
	{
		FConcertSyncPackageEvent PackageEvent;
		InSelectedActivity->EventPayload->GetTypedPayload(PackageEvent);
		checkf(!PackageEvent.Package.HasPackageData(), TEXT("UI should only request the package meta data because the package data is not useful and can be very large"));
		DisplayPackageDetails(*InSelectedActivity, PackageEvent.PackageRevision, PackageEvent.Package.Info);
	}
	else // Other activity types (lock/connection) don't have details panel.
	{
		SetDetailsPanelVisibility(NoDetailsPanel.Get());
	}
}

void SConcertSessionActivities::UpdateDetailsByRequest(TSharedPtr<FConcertSessionActivity> InSelectedActivity)
{
	if (InSelectedActivity->Activity.EventType == EConcertSyncActivityEventType::Transaction && GetTransactionEventFn.IsBound()) // A function is bound to get the transaction event?
	{
		SetDetailsPanelVisibility(LoadingDetailsPanel.Get());
		TWeakPtr<SConcertSessionActivities> WeakSelf = SharedThis(this);
		GetTransactionEventFn.Execute(*InSelectedActivity).Next([WeakSelf, InSelectedActivity](const TOptional<FConcertSyncTransactionEvent>& TransactionEvent)
		{
			if (TSharedPtr<SConcertSessionActivities> Self = WeakSelf.Pin()) // If 'this' object hasn't been deleted.
			{
				if (Self->GetSelectedActivity() == InSelectedActivity) // Ensure the activity is still selected.
				{
					if (TransactionEvent.IsSet() && TransactionEvent->Transaction.ExportedObjects.Num())
					{
						Self->DisplayTransactionDetails(*InSelectedActivity, TransactionEvent.GetValue().Transaction);
					}
					else
					{
						Self->SetDetailsPanelVisibility(Self->NoDetailsPanel.Get());
					}
				}
				// else -> The details panel is presenting information for another activity (or no activity).
			}
			// else -> The widget was deleted.
		});
	}
	else if (InSelectedActivity->Activity.EventType == EConcertSyncActivityEventType::Package && GetPackageEventFn.IsBound()) // A function is bound to get the package event?
	{
		SetDetailsPanelVisibility(LoadingDetailsPanel.Get());
		FConcertSyncPackageEventMetaData PackageEventMetaData;
		if (GetPackageEventFn.Execute(*InSelectedActivity, PackageEventMetaData))
		{
			DisplayPackageDetails(*InSelectedActivity, PackageEventMetaData.PackageRevision, PackageEventMetaData.PackageInfo);
		}
		else
		{
			SetDetailsPanelVisibility(NoDetailsPanel.Get());
		}
	}
	else if (InSelectedActivity->Activity.EventType == EConcertSyncActivityEventType::Replication)
	{
		// TODO UE-218508:
		SetDetailsPanelVisibility(NoDetailsPanel.Get());
	}
	else
	{
		SetDetailsPanelVisibility(NoDetailsPanel.Get());
	}
}

EConcertActivityFilterFlags SConcertSessionActivities::QueryActiveActivityFilters() const
{
	// The visibility attributes are externally provided. (In practice, they are controlled from the 'View Options' check boxes).
	EConcertActivityFilterFlags ActiveFlags = EConcertActivityFilterFlags::ShowAll;

	if (ConnectionActivitiesVisibility.Get() != EVisibility::Visible)
	{
		ActiveFlags |= EConcertActivityFilterFlags::HideConnectionActivities;
	}
	if (LockActivitiesVisibility.Get() != EVisibility::Visible)
	{
		ActiveFlags |= EConcertActivityFilterFlags::HideLockActivities;
	}
	if (PackageActivitiesVisibility.Get() != EVisibility::Visible)
	{
		ActiveFlags |= EConcertActivityFilterFlags::HidePackageActivities;
	}
	if (TransactionActivitiesVisibility.Get() != EVisibility::Visible)
	{
		ActiveFlags |= EConcertActivityFilterFlags::HideTransactionActivities;
	}
	if (ReplicationActivitiesVisibility.Get() != EVisibility::Visible)
	{
		ActiveFlags |= EConcertActivityFilterFlags::HideReplicationActivities;
	}
	if (IgnoredActivitiesVisibility.Get() != EVisibility::Visible)
	{
		ActiveFlags |= EConcertActivityFilterFlags::HideIgnoredActivities;
	}

	return ActiveFlags;
}

void SConcertSessionActivities::OnActivityFilterUpdated()
{
	// Try preserving the selected activity.
	TSharedPtr<FConcertSessionActivity> SelectedActivity = GetSelectedActivity();

	// Reset the list of displayed activities.
	Activities.Reset(AllActivities.Num());

	// Apply the filter.
	for (TSharedPtr<FConcertSessionActivity>& Activity : AllActivities)
	{
		if (PassesFilters(*Activity))
		{
			Activities.Add(Activity);
		}
	}

	// Restore/reset the selected activity.
	if (SelectedActivity && Activities.Contains(SelectedActivity))
	{
		ActivityView->SetItemSelection(SelectedActivity, true); // Restore previous selection.
		ActivityView->RequestScrollIntoView(SelectedActivity);
	}
	else if (bAutoScrollDesired && !bUserScrolling) // No activity was selected.
	{
		ActivityView->ScrollToBottom();
	}

	ActivityView->RequestListRefresh();
}

void SConcertSessionActivities::FetchActivities()
{
	if (!FetchActivitiesFn.IsBound()) // Not bound?
	{
		return; // The widget is expected to be populated/cleared externally using Append()/Reset()
	}

	bool bRefresh = false;

	// If they are still activities to fetch and the user scrolled down (or our nominal amount is not reached), request more from the server.
	if (!bAllActivitiesFetched && (Activities.Num() < DesiredActivitiesCount))
	{
		FText ErrorMsg;
		int32 FetchCount = 0; // The number of activities fetched in this iteration.
		int32 StartInsertPos = AllActivities.Num();

		bAllActivitiesFetched = FetchActivitiesFn.Execute(AllActivities, FetchCount, ErrorMsg);
		if (ErrorMsg.IsEmpty())
		{
			if (FetchCount) // New activities appended?
			{
				for (int32 Index = StartInsertPos; Index < AllActivities.Num(); ++Index) // Append the fetched activities
				{
					if (PassesFilters(*AllActivities[Index]))
					{
						Activities.Add(AllActivities[Index]);
						bRefresh = true;
					}

					if (AllActivities[Index]->Activity.bIgnored)
					{
						++IgnoredActivityNum;
					}
				}
			}
		}
		else
		{
			FAsyncTaskNotificationConfig NotificationConfig;
			NotificationConfig.bIsHeadless = false;
			NotificationConfig.bKeepOpenOnFailure = true;
			NotificationConfig.LogCategory = &LogConcert;

			FAsyncTaskNotification Notification(NotificationConfig);
			Notification.SetComplete(LOCTEXT("FetchError", "Failed to retrieve session activities"), ErrorMsg, /*Success*/ false);
		}
	}

	if (Activities.Num() && ActivityView->GetSelectedItems().Num() == 0)
	{
		ActivityView->SetItemSelection(Activities[0], true);
	}

	if (bRefresh)
	{
		if (bAutoScrollDesired && !bUserScrolling)
		{
			ActivityView->ScrollToBottom();
		}

		ActivityView->RequestListRefresh();
	}
}

void SConcertSessionActivities::Append(TSharedPtr<FConcertSessionActivity> Activity)
{
	if (Activity->Activity.bIgnored)
	{
		++IgnoredActivityNum;
	}

	AllActivities.Add(Activity);
	if (PassesFilters(*Activity))
	{
		Activities.Add(MoveTemp(Activity));

		if (bAutoScrollDesired && !bUserScrolling)
		{
			ActivityView->ScrollToBottom();
		}

		ActivityView->RequestListRefresh();
	}
}

void SConcertSessionActivities::RequestRefresh()
{
	ActivityView->RequestListRefresh();
}

void SConcertSessionActivities::ResetActivityList()
{
	Activities.Reset();
	AllActivities.Reset();
	ActivityView->RequestListRefresh();
	bAllActivitiesFetched = false;
	bUserScrolling = false;
	DesiredActivitiesCount = ActivitiesPerRequest;
	IgnoredActivityNum = 0;
}

bool SConcertSessionActivities::PassesFilters(const FConcertSessionActivity& Activity)
{
	if (Activity.Activity.EventType == EConcertSyncActivityEventType::Connection && ConnectionActivitiesVisibility.Get() != EVisibility::Visible) // Filter out 'connection' activities?
	{
		return false;
	}
	else if (Activity.Activity.EventType == EConcertSyncActivityEventType::Lock && LockActivitiesVisibility.Get() != EVisibility::Visible) // Filter out 'lock' activities?
	{
		return false;
	}
	else if (Activity.Activity.EventType == EConcertSyncActivityEventType::Package && PackageActivitiesVisibility.Get() != EVisibility::Visible) // Filter out 'package' activities?
	{
		return false;
	}
	else if (Activity.Activity.EventType == EConcertSyncActivityEventType::Transaction && TransactionActivitiesVisibility.Get() != EVisibility::Visible) // Filter out 'transaction' activities?
	{
		return false;
	}
	else if (Activity.Activity.EventType == EConcertSyncActivityEventType::Replication && ReplicationActivitiesVisibility.Get() != EVisibility::Visible) // Filter out 'replication' activities?
	{
		return false;
	}
	else if (Activity.Activity.bIgnored && IgnoredActivitiesVisibility.Get() != EVisibility::Visible) // Filter out 'ignored' activities?
	{
		return false;
	}

	return SearchTextFilter->PassesFilter(Activity);
}

FText SConcertSessionActivities::UpdateTextFilter(const FText& InFilterText)
{
	SearchTextFilter->SetRawFilterText(InFilterText);
	return SearchTextFilter->GetFilterErrorText();
}

void SConcertSessionActivities::OnColumnVisibilitySettingsChanged(const FColumnVisibilitySnapshot& ColumnSnapshot)
{
	TGuardValue<bool> GuardValue(bIsUpdatingColumnVisibility, true);
	UE::ConcertSharedSlate::RestoreColumnVisibilityState(GetHeaderRow().ToSharedRef(), ColumnSnapshot);
}

void SConcertSessionActivities::PopulateSearchStrings(const FConcertSessionActivity& Activity, TArray<FString>& OutSearchStrings)
{
	for (const SHeaderRow::FColumn& Column : HeaderRow->GetColumns())
	{
		const FActivityColumn& CastColumn = static_cast<const FActivityColumn&>(Column);
		CastColumn.ExecutePopulateSearchString(SharedThis(this), Activity, OutSearchStrings);
	}
}

TSharedPtr<FConcertSessionActivity> SConcertSessionActivities::GetSelectedActivity() const
{
	TArray<TSharedPtr<FConcertSessionActivity>> SelectedItems = ActivityView->GetSelectedItems();
	return SelectedItems.Num() ? SelectedItems[0] : nullptr;
}

TArray<TSharedPtr<FConcertSessionActivity>> SConcertSessionActivities::GetSelectedActivities() const
{
	return ActivityView->GetSelectedItems();
}

TSharedPtr<FConcertSessionActivity> SConcertSessionActivities::GetMostRecentActivity() const
{
	// NOTE: This function assumes that activities are sorted by ID. When used for recovery purpose,
	//       the activities are listed from the most recent to the oldest. When displaying a live
	//       session activity stream, the activities are listed from the oldest to the newest.

	if (AllActivities.Num()) // Ignore the filters.
	{
		if (AllActivities[0]->Activity.ActivityId > AllActivities.Last()->Activity.ActivityId)
		{
			return AllActivities[0]; // Listed from the latest to oldest.
		}
		return AllActivities.Last(); // Listed from the oldest to latest.
	}
	return nullptr; // The list is empty.
}

bool SConcertSessionActivities::IsLastColumn(const FName& ColumnId) const
{
	// Summary column is always visible and always the last.
	return ColumnId == UE::ConcertSharedSlate::ActivityColumn::SummaryColumnId; 
}

void SConcertSessionActivities::DisplayTransactionDetails(const FConcertSessionActivity& Activity, const FConcertTransactionEventBase& InTransaction)
{
	const FConcertSyncTransactionActivitySummary* Summary = Activity.ActivitySummary.Cast<FConcertSyncTransactionActivitySummary>();
	FString TransactionTitle = Summary ? Summary->TransactionTitle.ToString() : FString();

	FTransactionDiff TransactionDiff{ InTransaction.TransactionId, TransactionTitle };

	for (const FConcertExportedObject& ExportedObject : InTransaction.ExportedObjects)
	{
		FTransactionObjectDeltaChange DeltaChange;
		Algo::Transform(ExportedObject.PropertyDatas, DeltaChange.ChangedProperties, [](const FConcertSerializedPropertyData& PropertyData) { return PropertyData.PropertyName; });

		DeltaChange.bHasNameChange = ExportedObject.ObjectData.NewName != FName();
		DeltaChange.bHasOuterChange = ExportedObject.ObjectData.NewOuterPathName != FName();
		DeltaChange.bHasExternalPackageChange = ExportedObject.ObjectData.NewExternalPackageName != FName();
		DeltaChange.bHasPendingKillChange = ExportedObject.ObjectData.bIsPendingKill;

		FTransactionObjectId TransactionObjectId = ExportedObject.ObjectId.ToTransactionObjectId();
		TSharedPtr<FTransactionObjectEvent> Event = MakeShared<FTransactionObjectEvent>(InTransaction.TransactionId, InTransaction.OperationId, ETransactionObjectEventType::Finalized, ETransactionObjectChangeCreatedBy::TransactionRecord, 
			FTransactionObjectChange{ TransactionObjectId, (EObjectFlags)ExportedObject.ObjectId.ObjectPersistentFlags, MoveTemp(DeltaChange) }, nullptr);

		TransactionDiff.DiffMap.Emplace(TransactionObjectId.ObjectPathName, MoveTemp(Event));
	}

	if (ConcertReflectionDataProvider)
	{
		ConcertReflectionDataProvider.GetValue()->SetTransactionContext(InTransaction);
	}
	TransactionDetailsPanel->SetSelectedTransaction(MoveTemp(TransactionDiff));
	SetDetailsPanelVisibility(TransactionDetailsPanel.Get());
}

void SConcertSessionActivities::DisplayPackageDetails(const FConcertSessionActivity& Activity, int64 PackageRevision, const FConcertPackageInfo& PackageInfo)
{
	const TOptional<FConcertClientInfo> ClientInfo = GetActivityUserFn.IsBound()
		? GetActivityUserFn.Execute(Activity.Activity.EndpointId) : TOptional<FConcertClientInfo>{};
	PackageDetailsPanel->SetPackageInfo(PackageInfo, PackageRevision, ClientInfo ? ClientInfo->DisplayName : FString());
	SetDetailsPanelVisibility(PackageDetailsPanel.Get());
}

void SConcertSessionActivities::SetDetailsPanelVisibility(const SWidget* VisiblePanel)
{
	TransactionDetailsPanel->SetVisibility(VisiblePanel == TransactionDetailsPanel.Get() ? EVisibility::Visible : EVisibility::Collapsed);
	PackageDetailsPanel->SetVisibility(VisiblePanel == PackageDetailsPanel.Get() ? EVisibility::Visible : EVisibility::Collapsed);
	NoDetailsPanel->SetVisibility(VisiblePanel == NoDetailsPanel.Get() ? EVisibility::Visible : EVisibility::Collapsed);
	LoadingDetailsPanel->SetVisibility(VisiblePanel == LoadingDetailsPanel.Get() ? EVisibility::Visible : EVisibility::Collapsed);
}

TSharedRef<SWidget> FConcertSessionActivitiesOptions::MakeStatusBar(TAttribute<int32> Total, TAttribute<int32> Displayed)
{
	return SNew(STextBlock)
		.Text_Lambda([Total = MoveTemp(Total), Displayed = MoveTemp(Displayed)]()
		{
			if (Total.Get() == Displayed.Get())
			{
				return FText::Format(LOCTEXT("OperationCount", "{0} operations"), Total.Get());
			}
			return FText::Format(LOCTEXT("PartialOperationCount", "Showing {0} of {1} {1}|plural(one=operation,other=operations)"), Displayed.Get(), Total.Get());
		});
}

TSharedRef<SWidget> FConcertSessionActivitiesOptions::MakeViewOptionsComboButton(TOptional<FExtendContextMenu> ExtendMenu)
{
	return ConcertFrontendUtils::CreateViewOptionsComboButton(
		FOnGetContent::CreateRaw(this, &FConcertSessionActivitiesOptions::MakeMenuWidget, MoveTemp(ExtendMenu))
		);
}

TSharedRef<SWidget> FConcertSessionActivitiesOptions::MakeMenuWidget(TOptional<FExtendContextMenu> ExtendMenu)
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("DisplayRelativeTime", "Display Relative Time"),
		LOCTEXT("DisplayRelativeTime_Tooltip", "Displays Time Relative to the Current Time"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &FConcertSessionActivitiesOptions::OnOptionToggled, ConcertSessionActivityUtils::DisplayRelativeTimeCheckBoxId),
			FCanExecuteAction::CreateLambda([] { return true; }),
			FIsActionChecked::CreateLambda([this] { return bDisplayRelativeTime; })),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	if (bEnablePackageActivityFiltering)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ShowPackageActivities", "Show Package Activities"),
			LOCTEXT("ShowPackageActivities_Tooltip", "Displays create/save/rename/delete package events."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FConcertSessionActivitiesOptions::OnOptionToggled, ConcertSessionActivityUtils::ShowPackageActivitiesCheckBoxId),
				FCanExecuteAction::CreateLambda([] { return true; }),
				FIsActionChecked::CreateLambda([this] { return bDisplayPackageActivities; })),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}

	if (bEnableTransactionActivityFiltering)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ShowTransactionActivities", "Show Transaction Activities"),
			LOCTEXT("ShowTransactionActivities_Tooltip", "Displays changes performed on assets."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FConcertSessionActivitiesOptions::OnOptionToggled, ConcertSessionActivityUtils::ShowTransactionActivitiesCheckBoxId),
				FCanExecuteAction::CreateLambda([] { return true; }),
				FIsActionChecked::CreateLambda([this] { return bDisplayTransactionActivities; })),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}

	if (bEnableConnectionActivityFiltering)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ShowConnectionActivities", "Show Connection Activities"),
			LOCTEXT("ShowConnectionActivities_Tooltip", "Displays when client joined or left the session"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FConcertSessionActivitiesOptions::OnOptionToggled, ConcertSessionActivityUtils::ShowConnectionActivitiesCheckBoxId),
				FCanExecuteAction::CreateLambda([] { return true; }),
				FIsActionChecked::CreateLambda([this] { return bDisplayConnectionActivities; })),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}

	if (bEnableLockActivityFiltering)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ShowLockActivities", "Show Lock Activities"),
			LOCTEXT("ShowLockActivities_Tooltip", "Displays lock/unlock events"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FConcertSessionActivitiesOptions::OnOptionToggled, ConcertSessionActivityUtils::ShowLockActivitiesCheckBoxId),
				FCanExecuteAction::CreateLambda([] { return true; }),
				FIsActionChecked::CreateLambda([this] { return bDisplayLockActivities; })),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	
	if (bEnableReplicationActivityFiltering)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ShowReplicationActivities", "Show Replication Activities"),
			LOCTEXT("ShowReplicationActivities_Tooltip", "Displays replication events"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FConcertSessionActivitiesOptions::OnOptionToggled, ConcertSessionActivityUtils::ShowReplicationActivitiesCheckBoxId),
				FCanExecuteAction::CreateLambda([] { return true; }),
				FIsActionChecked::CreateLambda([this] { return bDisplayReplicationActivities; })),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}

	if (bEnableIgnoredActivityFiltering)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ShowIgnoredActivities", "Show Unrecoverable Activities"),
			LOCTEXT("ShowIgnoredActivities_Tooltip", "Displays activities that were recorded, but could not be recovered in this context."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FConcertSessionActivitiesOptions::OnOptionToggled, ConcertSessionActivityUtils::ShowIgnoredActivitiesCheckBoxId),
				FCanExecuteAction::CreateLambda([] { return true; }),
				FIsActionChecked::CreateLambda([this] { return bDisplayIgnoredActivities; })),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}

	if (ExtendMenu)
	{
		ExtendMenu->Execute(MenuBuilder);
	}
	
	return MenuBuilder.MakeWidget();
}

void FConcertSessionActivitiesOptions::OnOptionToggled(const FName CheckBoxId)
{
	if (CheckBoxId == ConcertSessionActivityUtils::DisplayRelativeTimeCheckBoxId)
	{
		bDisplayRelativeTime = !bDisplayRelativeTime;
	}
	else if (CheckBoxId == ConcertSessionActivityUtils::ShowConnectionActivitiesCheckBoxId)
	{
		bDisplayConnectionActivities = !bDisplayConnectionActivities;
	}
	else if (CheckBoxId == ConcertSessionActivityUtils::ShowLockActivitiesCheckBoxId)
	{
		bDisplayLockActivities = !bDisplayLockActivities;
	}
	else if (CheckBoxId == ConcertSessionActivityUtils::ShowPackageActivitiesCheckBoxId)
	{
		bDisplayPackageActivities = !bDisplayPackageActivities;
	}
	else if (CheckBoxId == ConcertSessionActivityUtils::ShowTransactionActivitiesCheckBoxId)
	{
		bDisplayTransactionActivities = !bDisplayTransactionActivities;
	}
	else if (CheckBoxId == ConcertSessionActivityUtils::ShowReplicationActivitiesCheckBoxId)
	{
		bDisplayReplicationActivities = !bDisplayReplicationActivities;
	}
	else if (CheckBoxId == ConcertSessionActivityUtils::ShowIgnoredActivitiesCheckBoxId)
	{
		bDisplayIgnoredActivities = !bDisplayIgnoredActivities;
	}	
}

#undef LOCTEXT_NAMESPACE
