// Copyright Epic Games, Inc. All Rights Reserved.

#include "SessionActivityUtils.h"

#include "ConcertSyncSessionTypes.h"
#include "Replication/Messages/ReplicationActivity.h"

#define LOCTEXT_NAMESPACE "SConcertSessionActivities"

FText UE::ConcertSharedSlate::Private::GetOperationName(const FConcertSessionActivity& Activity)
{
	static_assert(static_cast<uint8>(EConcertSyncActivityEventType::Count) == 6, "If you added an EConcertSyncActivityEventType entry, you may want to update this location");
	if (const FConcertSyncTransactionActivitySummary* TransactionSummary = Activity.ActivitySummary.Cast<FConcertSyncTransactionActivitySummary>())
	{
		return TransactionSummary->TransactionTitle;
	}

	if (const FConcertSyncPackageActivitySummary* PackageSummary = Activity.ActivitySummary.Cast<FConcertSyncPackageActivitySummary>())
	{
		auto GetSavedOperationNameFn = [](const FConcertSyncPackageActivitySummary* PackageSummary)
		{
			if (PackageSummary->bAutoSave)
			{
				return LOCTEXT("AutoSavePackageOperation", "Auto-Save Package");
			}
			return PackageSummary->bPreSave ? LOCTEXT("PreSavePackageOperation", "Pre-Save Package") : LOCTEXT("SavePackageOperation", "Save Package");
		};

		switch (PackageSummary->PackageUpdateType)
		{
		case EConcertPackageUpdateType::Added   : return LOCTEXT("NewPackageOperation",    "New Package");
		case EConcertPackageUpdateType::Deleted : return LOCTEXT("DeletePackageOperation", "Delete Package");
		case EConcertPackageUpdateType::Renamed : return LOCTEXT("RenamePackageOperation", "Rename Package");
		case EConcertPackageUpdateType::Saved   : return GetSavedOperationNameFn(PackageSummary);
		case EConcertPackageUpdateType::Dummy   : return LOCTEXT("DiscardPackageOperation", "Discard Changes");
		default: break;
		}
	}

	if (const FConcertSyncConnectionActivitySummary* ConnectionSummary = Activity.ActivitySummary.Cast<FConcertSyncConnectionActivitySummary>())
	{
		switch (ConnectionSummary->ConnectionEventType)
		{
		case EConcertSyncConnectionEventType::Connected:    return LOCTEXT("JoinOperation", "Join Session");
		case EConcertSyncConnectionEventType::Disconnected: return LOCTEXT("LeaveOperation", "Leave Session");
		default: break;
		}
	}

	if (const FConcertSyncLockActivitySummary* LockSummary = Activity.ActivitySummary.Cast<FConcertSyncLockActivitySummary>())
	{
		switch (LockSummary->LockEventType)
		{
		case EConcertSyncLockEventType::Locked:   return LOCTEXT("LockOperation", "Lock");
		case EConcertSyncLockEventType::Unlocked: return LOCTEXT("UnlockOperation", "Unlock");
		default: break;
		}
	}

	if (const FConcertSyncReplicationActivitySummary* ActivitySummary = Activity.ActivitySummary.Cast<FConcertSyncReplicationActivitySummary>())
	{
		return ActivitySummary->ToDisplayTitle();
	}

	return FText::GetEmpty();
}

FText UE::ConcertSharedSlate::Private::GetPackageName(const FConcertSessionActivity& Activity)
{
	if (const FConcertSyncPackageActivitySummary* Summary = Activity.ActivitySummary.Cast<FConcertSyncPackageActivitySummary>())
	{
		return FText::FromName(Summary->PackageName);
	}

	if (const FConcertSyncTransactionActivitySummary* Summary = Activity.ActivitySummary.Cast<FConcertSyncTransactionActivitySummary>())
	{
		return FText::FromName(Summary->PrimaryPackageName);
	}

	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
