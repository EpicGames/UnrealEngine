// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


class FText;

struct FConcertSessionActivity;

namespace UE::ConcertSharedSlate::Private
{
	FText GetOperationName(const FConcertSessionActivity& Activity);
	FText GetPackageName(const FConcertSessionActivity& Activity);
}
