// Copyright Epic Games, Inc. All Rights Reserved.

#include "Session/Browser/SConcertSessionBrowser.h"

#include "Framework/Views/TableViewMetadata.h"
#include "Session/Browser/ConcertBrowserUtils.h"
#include "Session/Browser/ConcertSessionBrowserSettings.h"
#include "Session/Browser/Items/ConcertArchivedGroupTreeItem.h"
#include "Session/Browser/IConcertSessionBrowserController.h"
#include "Session/Browser/Items/ConcertGroupTreeItem.h"
#include "Session/Browser/SNewSessionRow.h"
#include "Session/Browser/Items/ConcertTreeItem.h"
#include "Session/Browser/SSaveRestoreSessionRow.h"
#include "Session/Browser/SSessionRow.h"
#include "Session/Browser/SSessionGroupRow.h"

#include "Algo/ForEach.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/TextFilter.h"
#include "Session/Browser/Items/ConcertActiveGroupTreeItem.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SSeparator.h"
#include "Styling/ToolBarStyle.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "SConcertBrowser"

const FName SConcertSessionBrowser::ControlButtonExtensionHooks::BeforeSeparator("BeforeSeparator");
const FName SConcertSessionBrowser::ControlButtonExtensionHooks::Separator("Separator");
const FName SConcertSessionBrowser::ControlButtonExtensionHooks::AfterSeparator("AfterSeparator");

const FName SConcertSessionBrowser::SessionContextMenuExtensionHooks::ManageSession("ManageSession");

void SConcertSessionBrowser::Construct(const FArguments& InArgs, TSharedRef<IConcertSessionBrowserController> InController, TSharedPtr<FText> InSearchText)
{
	Controller = InController;

	// Reload the persistent settings, such as the filters.
	PersistentSettings = GetMutableDefault<UConcertSessionBrowserSettings>();

	DefaultSessionName = InArgs._DefaultSessionName;
	DefaultServerUrl = InArgs._DefaultServerURL;

	ExtendSessionContextMenu = InArgs._ExtendSessionContextMenu;
	OnSessionClicked = InArgs._OnSessionClicked;
	OnLiveSessionDoubleClicked = InArgs._OnLiveSessionDoubleClicked;
	OnArchivedSessionDoubleClicked = InArgs._OnArchivedSessionDoubleClicked;
	if (!OnArchivedSessionDoubleClicked.IsBound())
	{
		OnArchivedSessionDoubleClicked = FSessionDelegate::CreateSP(this, &SConcertSessionBrowser::InsertRestoreSessionAsEditableRowInternal);
	}
	PostRequestedDeleteSession = InArgs._PostRequestedDeleteSession;
	AskUserToDeleteSessions = InArgs._AskUserToDeleteSessions;

	// Setup search filter.
	SearchedText = InSearchText; // Reload a previous search text (in any). Useful to remember searched text between join/leave sessions, but not persistent if the tab is closed.
	SearchTextFilter = MakeShared<TTextFilter<const FConcertSessionTreeItem&>>(TTextFilter<const FConcertSessionTreeItem&>::FItemToStringArray::CreateSP(this, &SConcertSessionBrowser::PopulateSearchStrings));
	SearchTextFilter->OnChanged().AddSP(this, &SConcertSessionBrowser::RefreshSessionList);

	auto GetSessions = [this]() { return Sessions; };
	ActiveSessionsGroupItem = MakeShared<FConcertActiveGroupTreeItem>(GetSessions);
	ArchivedSessionsGroupItem = MakeShared<FConcertArchivedGroupTreeItem>(GetSessions);
	TreeItemSource = { ActiveSessionsGroupItem, ArchivedSessionsGroupItem };

	ChildSlot
	[
		MakeBrowserContent(InArgs)
	];
	// By default have them expanded
	SessionsView->SetItemExpansion(ActiveSessionsGroupItem, true);
	SessionsView->SetItemExpansion(ArchivedSessionsGroupItem, true);

	if (!SearchedText->IsEmpty())
	{
		SearchBox->SetText(*SearchedText); // This trigger the chain of actions to apply the search filter.
	}
}

TSharedRef<SWidget> SConcertSessionBrowser::MakeBrowserContent(const FArguments& InArgs)
{
	TSharedRef<SWidget> SessionTable = MakeSessionTableView(InArgs);
	return SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
		.Padding(FMargin(1.0f, 2.0f))
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				MakeControlBar(InArgs)
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SSeparator)
				.Thickness(FCoreStyle::Get().GetWidgetStyle<FToolBarStyle>("ToolBar").SeparatorThickness)
				.SeparatorImage(&FCoreStyle::Get().GetWidgetStyle<FToolBarStyle>("ToolBar").SeparatorBrush)
			]

			// The search text.
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 4.f, 0.f, 4.f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SAssignNew(SearchBox, SSearchBox)
					.HintText(LOCTEXT("SearchHint", "Search Session"))
					.OnTextChanged(this, &SConcertSessionBrowser::OnSearchTextChanged)
					.OnTextCommitted(this, &SConcertSessionBrowser::OnSearchTextCommitted)
					.DelayChangeNotificationsWhileTyping(true)
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					ConcertFrontendUtils::CreateViewOptionsComboButton(FOnGetContent::CreateSP(this, &SConcertSessionBrowser::MakeViewOptionsComboButtonMenu))
				]
			]

			// Session list.
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(1.0f, 2.0f)
			[
				InArgs._ExtendSessionTable.IsBound() ? InArgs._ExtendSessionTable.Execute(SessionTable) : SessionTable
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(2.0f, 0.0f))
			[
				SNew(SSeparator)
			]

			// Session Count/View options filter.
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(2.0f, 0.0f))
			[
				MakeSessionTableFooter()
			]
		];
}


void SConcertSessionBrowser::OnGetChildren(TSharedPtr<FConcertTreeItem> InItem, TArray<TSharedPtr<FConcertTreeItem>>& OutChildren)
{
	return InItem->GetChildren(OutChildren);
}

namespace UE::SessionBrowser::Private
{

template <typename SessionType>
constexpr FConcertSessionTreeItem::EType GetEType()
{
	if constexpr(std::is_same<IConcertSessionBrowserController::FActiveSessionInfo,SessionType>::value)
	{
		return FConcertSessionTreeItem::EType::ActiveSession;
	}
	else
	{
		return FConcertSessionTreeItem::EType::ArchivedSession;
	}
}

template <typename SessionType>
static FConcertSessionTreeItem GenerateSessionItem(const SessionType& Session)
{
	FText Version = LOCTEXT("ConcertEmptyVersion", "Unset");
	if (Session.SessionInfo.VersionInfos.Num()>0)
	{
		const FConcertSessionVersionInfo& VersionInfo = Session.SessionInfo.VersionInfos.Last();
		Version = VersionInfo.AsText();
	}
	return FConcertSessionTreeItem(
		GetEType<SessionType>(),
		Session.ServerInfo.AdminEndpointId,
		Session.SessionInfo.SessionId,
		Session.SessionInfo.SessionName,
		Session.ServerInfo.ServerName,
		Session.SessionInfo.Settings.ProjectName,
		Version.ToString(),
		Session.ServerInfo.ServerFlags,
		Session.SessionInfo.GetLastModified(),
		{}
	);
}

}

void SConcertSessionBrowser::RefreshSessionList()
{
	// Remember the selected instances (if any).
	TArray<TSharedPtr<FConcertSessionTreeItem>> SelectedItems = GetSelectedItems();
	TArray<TSharedPtr<FConcertSessionTreeItem>> ReselectedItems;
	TSharedPtr<FConcertSessionTreeItem> NewEditableRowParent;

	// Predicate returning true if the specified item should be re-selected.
	auto IsSelected = [&SelectedItems](const FConcertSessionTreeItem& Item)
	{
		return SelectedItems.ContainsByPredicate([Item](const TSharedPtr<FConcertSessionTreeItem>& Visited) { return *Visited == Item; });
	};
	// Matches the object instances before the update to the new instance after the update.
	auto ReconcileObjectInstances = [&](TSharedPtr<FConcertSessionTreeItem> NewItem)
	{
		if (IsSelected(*NewItem))
		{
			ReselectedItems.Add(NewItem);
		}
		else if (EditableSessionRowParent.IsValid() && !NewEditableRowParent.IsValid() && *EditableSessionRowParent == *NewItem)
		{
			NewEditableRowParent = NewItem;
		}
	};
	auto ConvertSession = [&](const auto& Session)
	{
		FConcertSessionTreeItem NewItem = UE::SessionBrowser::Private::GenerateSessionItem(Session);
		if (!IsFilteredOut(NewItem))
		{
			const TSharedPtr<FConcertSessionTreeItem> Item = MakeShared<FConcertSessionTreeItem>(MoveTemp(NewItem));
			AddSession(Item);
			ReconcileObjectInstances(Item);
		}
	};
	ResetSessions();

	TArray<IConcertSessionBrowserController::FActiveSessionInfo> ActiveSessions = GetController()->GetActiveSessions();
	TArray<IConcertSessionBrowserController::FArchivedSessionInfo> ArchivedSessions = GetController()->GetArchivedSessions();
	Algo::ForEach(ActiveSessions, ConvertSession);
	Algo::ForEach(ArchivedSessions, ConvertSession);

	// Restore the editable row state. (SortSessionList() below will ensure the parent/child relationship)
	EditableSessionRowParent = NewEditableRowParent;
	if (EditableSessionRow.IsValid())
	{
		if (EditableSessionRow->Type == FConcertSessionTreeItem::EType::NewSession)
		{
			InsertSessionAtBeginning(EditableSessionRow); // Always put 'new session' row at the top.
		}
		else if (EditableSessionRowParent.IsValid())
		{
			AddSession(EditableSessionRow); // SortSessionList() called below will ensure the correct parent/child order.
		}
	}

	// Restore previous selection.
	if (ReselectedItems.Num())
	{
		for (const TSharedPtr<FConcertSessionTreeItem>& Item : ReselectedItems)
		{
			SessionsView->SetItemSelection(Item, true);
		}
	}

	SortSessionList();
	SessionsView->RequestListRefresh();
}

TSharedPtr<IConcertSessionBrowserController> SConcertSessionBrowser::GetController() const
{
	TSharedPtr<IConcertSessionBrowserController> Result = Controller.Pin();
	check(Result);
	return Result;
}

bool SConcertSessionBrowser::IsGroupItem(const TSharedPtr<FConcertTreeItem>& TreeItem) const
{
	return TreeItem == ActiveSessionsGroupItem || TreeItem == ArchivedSessionsGroupItem;
}

void SConcertSessionBrowser::ResetSessions()
{
	Sessions.Reset();
}

void SConcertSessionBrowser::AddSession(TSharedPtr<FConcertSessionTreeItem> Session)
{
	Sessions.Add(Session);
}

void SConcertSessionBrowser::InsertSessionAtBeginning(TSharedPtr<FConcertSessionTreeItem> NewSession)
{
	
	Sessions.Insert(NewSession, 0);
}

void SConcertSessionBrowser::InsertSessionAfterParent(TSharedPtr<FConcertSessionTreeItem> NewSession, TSharedPtr<FConcertSessionTreeItem> Parent)
{
	const int32 TreeParentIndex = Sessions.IndexOfByKey(Parent);
	if (TreeParentIndex != INDEX_NONE)
	{
		Sessions.Insert(NewSession, TreeParentIndex + 1);
	}
}

SConcertSessionBrowser::EInsertPosition SConcertSessionBrowser::InsertSessionAfterParentIfAvailableOrAtBeginning(TSharedPtr<FConcertSessionTreeItem> NewSession, TSharedPtr<FConcertSessionTreeItem> Parent)
{
	const int32 ParentIndex = Sessions.IndexOfByKey(Parent);
	const bool bPlaceAfterParent = ParentIndex != INDEX_NONE; 
	Sessions.Insert(NewSession, bPlaceAfterParent ? ParentIndex + 1 : 0);
	return bPlaceAfterParent ? EInsertPosition::AfterParent : EInsertPosition::AtBeginning;
}

void SConcertSessionBrowser::RemoveSession(TSharedPtr<FConcertSessionTreeItem> Session)
{
	Sessions.RemoveSingle(Session);
}

void SConcertSessionBrowser::SortSessionList()
{
	check(!PrimarySortedColumn.IsNone()); // Should always have a primary column. User cannot clear this one.

	auto Compare = [this](const TSharedPtr<FConcertSessionTreeItem>& Lhs, const TSharedPtr<FConcertSessionTreeItem>& Rhs, const FName& ColName, EColumnSortMode::Type SortMode)
	{
		if (Lhs->Type == FConcertSessionTreeItem::EType::NewSession) // Always keep editable 'new session' row at the top.
		{
			return true;
		}
		if (Rhs->Type == FConcertSessionTreeItem::EType::NewSession)
		{
			return false;
		}

		if (ColName == ConcertBrowserUtils::SessionColName)
		{
			return SortMode == EColumnSortMode::Ascending ? Lhs->SessionName < Rhs->SessionName : Lhs->SessionName > Rhs->SessionName;
		}
		if (ColName == ConcertBrowserUtils::ServerColName)
		{
			return SortMode == EColumnSortMode::Ascending ? Lhs->ServerName < Rhs->ServerName : Lhs->ServerName > Rhs->ServerName;
		}
		if (ColName == ConcertBrowserUtils::ProjectColName)
		{
			return SortMode == EColumnSortMode::Ascending ? Lhs->ProjectName < Rhs->ProjectName : Lhs->ProjectName > Rhs->ProjectName;
		}
		if (ColName == ConcertBrowserUtils::VersionColName)
		{
			return SortMode == EColumnSortMode::Ascending ? Lhs->ProjectVersion < Rhs->ProjectVersion : Lhs->ProjectVersion > Rhs->ProjectVersion;
		}
		if (ColName == ConcertBrowserUtils::LastModifiedColName)
		{
			return SortMode == EColumnSortMode::Ascending ? Lhs->LastModified < Rhs->LastModified : Lhs->LastModified > Rhs->LastModified;
		}
		else
		{
			// Did you add a new column?
			checkNoEntry();
			return false;
		}
	};

	Sessions.Sort([&](const TSharedPtr<FConcertSessionTreeItem>& Lhs, const TSharedPtr<FConcertSessionTreeItem>& Rhs)
	{
		if (Compare(Lhs, Rhs, PrimarySortedColumn, PrimarySortMode))
		{
			return true; // Lhs must be before Rhs based on the primary sort order.
		}
		if (Compare(Rhs, Lhs, PrimarySortedColumn, PrimarySortMode)) // Invert operands order (goal is to check if operands are equal or not)
		{
			return false; // Rhs must be before Lhs based on the primary sort.
		}
		else // Lhs == Rhs on the primary column, need to order accoding the secondary column if one is set.
		{
			return SecondarySortedColumn.IsNone() ? false : Compare(Lhs, Rhs, SecondarySortedColumn, SecondarySortMode);
		}
	});

	EnsureEditableParentChildOrder();
}

void SConcertSessionBrowser::OnSearchTextChanged(const FText& InFilterText)
{
	SearchTextFilter->SetRawFilterText(InFilterText);
	SearchBox->SetError( SearchTextFilter->GetFilterErrorText() );
	*SearchedText = InFilterText;

	bRefreshSessionFilter = true;
}

void SConcertSessionBrowser::OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type CommitType)
{
	if (!InFilterText.EqualTo(*SearchedText))
	{
		OnSearchTextChanged(InFilterText);
	}
}

void SConcertSessionBrowser::PopulateSearchStrings(const FConcertSessionTreeItem& Item, TArray<FString>& OutSearchStrings) const
{
	OutSearchStrings.Add(Item.ServerName);
	OutSearchStrings.Add(Item.SessionName);
}

bool SConcertSessionBrowser::IsFilteredOut(const FConcertSessionTreeItem& Item) const
{
	bool bIsDefaultServer = LastDefaultServerUrl.IsEmpty() || Item.ServerName == LastDefaultServerUrl;

	return (!PersistentSettings->bShowActiveSessions && (Item.Type == FConcertSessionTreeItem::EType::ActiveSession || Item.Type == FConcertSessionTreeItem::EType::SaveSession)) ||
		   (!PersistentSettings->bShowArchivedSessions && (Item.Type == FConcertSessionTreeItem::EType::ArchivedSession || Item.Type == FConcertSessionTreeItem::EType::RestoreSession)) ||
		   (PersistentSettings->bShowDefaultServerSessionsOnly && !bIsDefaultServer) ||
		   !SearchTextFilter->PassesFilter(Item);
}

FText SConcertSessionBrowser::HighlightSearchText() const
{
	return *SearchedText;
}

TSharedRef<SWidget> SConcertSessionBrowser::MakeControlBar(const FArguments& InArgs)
{
	return SNew(SHorizontalBox)
		// The New/Join/Restore/Delete/Archive buttons
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			MakeButtonBar(InArgs)
		]

		// The search text.
		+SHorizontalBox::Slot()
		.FillWidth(1.0)
		[
			SNew(SSpacer)
		]

		// Optional: everything to the right of the search bar, e.g. user name and settings combo button
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			InArgs._RightOfControlButtons.Widget
		];
}

TSharedRef<SWidget> SConcertSessionBrowser::MakeButtonBar(const FArguments& InArgs)
{
	TSharedRef<FExtender> Extender = MakeShared<FExtender>();
	InArgs._ExtendControllButtons.ExecuteIfBound(Extender.Get());
	FToolBarBuilder RowBuilder(nullptr, FMultiBoxCustomization::None, Extender);
	
	// New Session
	if (GetController()->CanEverCreateSessions())
	{
		RowBuilder.BeginSection(ControlButtonExtensionHooks::BeforeSeparator);
		RowBuilder.AddWidget(
			ConcertBrowserUtils::MakeIconButton(
				FConcertFrontendStyle::Get()->GetBrush("Concert.NewSession"),
				LOCTEXT("NewButtonTooltip", "Create a new session"),
				TAttribute<bool>(this, &SConcertSessionBrowser::IsNewButtonEnabledInternal),
				FOnClicked::CreateSP(this, &SConcertSessionBrowser::OnNewButtonClicked)
			)
		);
		RowBuilder.EndSection();
		RowBuilder.AddSeparator(ControlButtonExtensionHooks::Separator);
	}
	RowBuilder.BeginSection(ControlButtonExtensionHooks::AfterSeparator);
	
	// Restore (Share the same slot as Join)
	RowBuilder.AddWidget(
		ConcertBrowserUtils::MakeIconButton(
			FConcertFrontendStyle::Get()->GetBrush("Concert.RestoreSession"),
			LOCTEXT("RestoreButtonTooltip", "Restore the selected session"),
			TAttribute<bool>(this, &SConcertSessionBrowser::IsRestoreButtonEnabledInternal),
			FOnClicked::CreateSP(this, &SConcertSessionBrowser::OnRestoreButtonClicked)
			)
	);
	// Archive.
	RowBuilder.AddWidget(
		ConcertBrowserUtils::MakeIconButton(FConcertFrontendStyle::Get()->GetBrush("Concert.ArchiveSession"), LOCTEXT("ArchiveButtonTooltip", "Archive the selected session"),
			TAttribute<bool>(this, &SConcertSessionBrowser::IsArchiveButtonEnabledInternal),
			FOnClicked::CreateSP(this, &SConcertSessionBrowser::OnArchiveButtonClicked))
	);
	// Delete.
	RowBuilder.AddWidget(
		ConcertBrowserUtils::MakeIconButton(FConcertFrontendStyle::Get()->GetBrush("Concert.DeleteSession"), LOCTEXT("DeleteButtonTooltip", "Delete the selected session if permitted"),
			TAttribute<bool>(this, &SConcertSessionBrowser::IsDeleteButtonEnabledInternal),
			FOnClicked::CreateSP(this, &SConcertSessionBrowser::OnDeleteButtonClicked))
	);
	RowBuilder.EndSection();

	return RowBuilder.MakeWidget();
}

TSharedRef<SWidget> SConcertSessionBrowser::MakeSessionTableView(const FArguments& InArgs)
{
	using namespace UE::ConcertSharedSlate;

	PrimarySortedColumn = ConcertBrowserUtils::SessionColName;
	PrimarySortMode = EColumnSortMode::Ascending;
	SecondarySortedColumn = FName();
	SecondarySortMode = EColumnSortMode::Ascending;

	SessionsView = SNew(STreeView<TSharedPtr<FConcertTreeItem>>)
		.TreeItemsSource(&TreeItemSource)
		.OnGenerateRow(this, &SConcertSessionBrowser::OnGenerateSessionRowWidget)
		.OnGetChildren(this, &SConcertSessionBrowser::OnGetChildren)
		.SelectionMode(ESelectionMode::Multi)
		.OnSelectionChanged(this, &SConcertSessionBrowser::OnSessionSelectionChanged)
		.OnContextMenuOpening(this, &SConcertSessionBrowser::MakeTableContextualMenu)
		.HeaderRow(
			SAssignNew(SessionHeaderRow, SHeaderRow)
			.OnHiddenColumnsListChanged_Lambda([this, SaveCallback = InArgs._SaveColumnVisibilitySnapshot]()
			{
				if (SaveCallback.IsBound())
				{
					const FColumnVisibilitySnapshot Snapshot = SnapshotColumnVisibilityState(SessionHeaderRow.ToSharedRef());
					SaveCallback.Execute(Snapshot);
				}
			})

			+SHeaderRow::Column(ConcertBrowserUtils::SessionColName)
			.DefaultLabel(LOCTEXT("SessioName", "Session"))
			.FillWidth(2.f)
			.SortPriority(this, &SConcertSessionBrowser::GetColumnSortPriority, ConcertBrowserUtils::SessionColName)
			.SortMode(this, &SConcertSessionBrowser::GetColumnSortMode, ConcertBrowserUtils::SessionColName)
			.OnSort(this, &SConcertSessionBrowser::OnColumnSortModeChanged)
			.ShouldGenerateWidget(true)

			+SHeaderRow::Column(ConcertBrowserUtils::ServerColName)
			.DefaultLabel(LOCTEXT("Server", "Server"))
			.FillWidth(1.f)
			.SortPriority(this, &SConcertSessionBrowser::GetColumnSortPriority, ConcertBrowserUtils::ServerColName)
			.SortMode(this, &SConcertSessionBrowser::GetColumnSortMode, ConcertBrowserUtils::ServerColName)
			.OnSort(this, &SConcertSessionBrowser::OnColumnSortModeChanged)
			
			+SHeaderRow::Column(ConcertBrowserUtils::ProjectColName)
			.DefaultLabel(LOCTEXT("ProjectCol", "Project"))
			.FillWidth(1.f)
			.SortPriority(this, &SConcertSessionBrowser::GetColumnSortPriority, ConcertBrowserUtils::ProjectColName)
			.SortMode(this, &SConcertSessionBrowser::GetColumnSortMode, ConcertBrowserUtils::ProjectColName)
			.OnSort(this, &SConcertSessionBrowser::OnColumnSortModeChanged)
			
			+SHeaderRow::Column(ConcertBrowserUtils::VersionColName)
			.DefaultLabel(LOCTEXT("Version", "Version"))
			.FillWidth(1.f)
			.SortPriority(this, &SConcertSessionBrowser::GetColumnSortPriority, ConcertBrowserUtils::VersionColName)
			.SortMode(this, &SConcertSessionBrowser::GetColumnSortMode, ConcertBrowserUtils::VersionColName)
			.OnSort(this, &SConcertSessionBrowser::OnColumnSortModeChanged)
			
			+SHeaderRow::Column(ConcertBrowserUtils::LastModifiedColName)
			.DefaultLabel(LOCTEXT("LastModified", "Last Modified"))
			.FillWidth(1.f)
			.DefaultTooltip(LOCTEXT("LastModifiedTooltip", "The local time the session's directory was last modified"))
			.SortPriority(this, &SConcertSessionBrowser::GetColumnSortPriority, ConcertBrowserUtils::LastModifiedColName)
			.SortMode(this, &SConcertSessionBrowser::GetColumnSortMode, ConcertBrowserUtils::LastModifiedColName)
			.OnSort(this, &SConcertSessionBrowser::OnColumnSortModeChanged)
			);

	RestoreColumnVisibilityState(SessionHeaderRow.ToSharedRef(), InArgs._ColumnVisibilitySnapshot);
	return SessionsView.ToSharedRef();
}

EColumnSortMode::Type SConcertSessionBrowser::GetColumnSortMode(const FName ColumnId) const
{
	if (ColumnId == PrimarySortedColumn)
	{
		return PrimarySortMode;
	}
	if (ColumnId == SecondarySortedColumn)
	{
		return SecondarySortMode;
	}

	return EColumnSortMode::None;
}

EColumnSortPriority::Type SConcertSessionBrowser::GetColumnSortPriority(const FName ColumnId) const
{
	if (ColumnId == PrimarySortedColumn)
	{
		return EColumnSortPriority::Primary;
	}
	else if (ColumnId == SecondarySortedColumn)
	{
		return EColumnSortPriority::Secondary;
	}

	return EColumnSortPriority::Max; // No specific priority.
}

void SConcertSessionBrowser::OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	if (SortPriority == EColumnSortPriority::Primary)
	{
		PrimarySortedColumn = ColumnId;
		PrimarySortMode = InSortMode;

		if (ColumnId == SecondarySortedColumn) // Cannot be primary and secondary at the same time.
		{
			SecondarySortedColumn = FName();
			SecondarySortMode = EColumnSortMode::None;
		}
	}
	else if (SortPriority == EColumnSortPriority::Secondary)
	{
		SecondarySortedColumn = ColumnId;
		SecondarySortMode = InSortMode;
	}

	SortSessionList();
	SessionsView->RequestListRefresh();
}

void SConcertSessionBrowser::EnsureEditableParentChildOrder()
{
	// This is for Archiving or Restoring a session. We keep the editable row below the session to archive or restore and visually link them with small wires in UI.
	if (EditableSessionRowParent.IsValid())
	{
		check(EditableSessionRow.IsValid());
		RemoveSession(EditableSessionRow);
		
		InsertSessionAfterParent(EditableSessionRow, EditableSessionRowParent); // Insert the 'child' below its parent.
	}
}

TSharedRef<ITableRow> SConcertSessionBrowser::OnGenerateSessionRowWidget(TSharedPtr<FConcertTreeItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	if (IsGroupItem(Item))
	{
		return MakeGroupRowWidget(Item, OwnerTable);
	}

	const TSharedPtr<FConcertSessionTreeItem> SessionItem = StaticCastSharedPtr<FConcertSessionTreeItem>(Item);
	switch (SessionItem->Type)
	{
		case FConcertSessionTreeItem::EType::ActiveSession:
			return MakeActiveSessionRowWidget(SessionItem, OwnerTable);

		case FConcertSessionTreeItem::EType::ArchivedSession:
			return MakeArchivedSessionRowWidget(SessionItem, OwnerTable);

		case FConcertSessionTreeItem::EType::NewSession:
			return MakeNewSessionRowWidget(SessionItem, OwnerTable);

		case FConcertSessionTreeItem::EType::RestoreSession:
			return MakeRestoreSessionRowWidget(SessionItem, OwnerTable);

		default:
			check(SessionItem->Type == FConcertSessionTreeItem::EType::SaveSession);
			return MakeSaveSessionRowWidget(SessionItem, OwnerTable);
	}
}

TSharedRef<ITableRow> SConcertSessionBrowser::MakeActiveSessionRowWidget(const TSharedPtr<FConcertSessionTreeItem>& ActiveItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	const TOptional<FConcertSessionInfo> SessionInfo = GetController()->GetActiveSessionInfo(ActiveItem->ServerAdminEndpointId, ActiveItem->SessionId);

	// Add an 'Active Session' row. Clicking the row icon joins the session.
	return SNew(SSessionRow, ActiveItem, OwnerTable)
		.Padding(FConcertFrontendStyle::Get()->GetMargin("Concert.SessionRowPadding"))
		.OnDoubleClickFunc([this](const TSharedPtr<FConcertSessionTreeItem>& Item) { OnLiveSessionDoubleClicked.ExecuteIfBound(Item); })
		.OnRenameFunc([this](const TSharedPtr<FConcertSessionTreeItem>& Item, const FString& NewName) { RequestRenameSession(Item, NewName); })
		.IsDefaultSession([this](TSharedPtr<FConcertSessionTreeItem> ItemPin)
		{
			return ItemPin->Type == FConcertSessionTreeItem::EType::ActiveSession
				&& DefaultSessionName.IsSet() && ItemPin->SessionName == DefaultSessionName.Get()
				&& DefaultServerUrl.IsSet() && ItemPin->ServerName == DefaultServerUrl.Get();
		})
		.HighlightText(this, &SConcertSessionBrowser::HighlightSearchText)
		.ToolTipText(SessionInfo ? SessionInfo->ToDisplayString() : FText::GetEmpty())
		.IsSelected_Lambda([this, ActiveItem]() { return SessionsView->GetSelectedItems().Num() == 1 && SessionsView->GetSelectedItems()[0] == ActiveItem; });
}

TSharedRef<ITableRow> SConcertSessionBrowser::MakeArchivedSessionRowWidget(const TSharedPtr<FConcertSessionTreeItem>& ArchivedItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	const TOptional<FConcertSessionInfo> SessionInfo = GetController()->GetArchivedSessionInfo(ArchivedItem->ServerAdminEndpointId, ArchivedItem->SessionId);

	// Add an 'Archived Session' row. Clicking the row icon adds a 'Restore as' row to the table.
	return SNew(SSessionRow, ArchivedItem, OwnerTable)
		.Padding(FConcertFrontendStyle::Get()->GetMargin("Concert.SessionRowPadding"))
		.OnDoubleClickFunc([this](const TSharedPtr<FConcertSessionTreeItem>& Item) { OnArchivedSessionDoubleClicked.Execute(Item); })
		.OnRenameFunc([this](const TSharedPtr<FConcertSessionTreeItem>& Item, const FString& NewName) { RequestRenameSession(Item, NewName); })
		.IsDefaultSession([this](TSharedPtr<FConcertSessionTreeItem> ItemPin)
		{
			return ItemPin->Type == FConcertSessionTreeItem::EType::ActiveSession
				&& DefaultSessionName.IsSet() && ItemPin->SessionName == DefaultSessionName.Get()
				&& DefaultServerUrl.IsSet() && ItemPin->ServerName == DefaultServerUrl.Get();
		})
		.HighlightText(this, &SConcertSessionBrowser::HighlightSearchText)
		.ToolTipText(SessionInfo ? SessionInfo->ToDisplayString() : FText::GetEmpty())
		.IsSelected_Lambda([this, ArchivedItem]() { return SessionsView->GetSelectedItems().Num() == 1 && SessionsView->GetSelectedItems()[0] == ArchivedItem; });
}

TSharedRef<ITableRow> SConcertSessionBrowser::MakeNewSessionRowWidget(const TSharedPtr<FConcertSessionTreeItem>& NewItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	// Add an editable 'New Session' row in the table to let user pick a name and a server.
	TAttribute<FString> DefaultServerUrlArg = DefaultServerUrl.IsSet()
		? TAttribute<FString>::Create([this](){ return DefaultServerUrl.Get(); }) : TAttribute<FString>();
	return SNew(SNewSessionRow, NewItem, OwnerTable)
		.Padding(FConcertFrontendStyle::Get()->GetMargin("Concert.SessionRowPadding"))
		.GetServerFunc([this]() { return GetController()->GetServers(); }) // Let the row pull the servers for the combo box.
		.OnAcceptFunc([this](const TSharedPtr<FConcertSessionTreeItem>& Item) { RequestCreateSession(Item); }) // Accepting creates the session.
		.OnDeclineFunc([this](const TSharedPtr<FConcertSessionTreeItem>& Item) { RemoveSessionRow(Item); })  // Declining removes the editable 'new session' row from the view.
		.HighlightText(this, &SConcertSessionBrowser::HighlightSearchText)
		.DefaultServerURL(DefaultServerUrlArg);
}

TSharedRef<ITableRow> SConcertSessionBrowser::MakeSaveSessionRowWidget(const TSharedPtr<FConcertSessionTreeItem>& SaveItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	// Add an editable 'Save Session' row in the table to let the user enter an archive name.
	return SNew(SSaveRestoreSessionRow, SaveItem, OwnerTable)
		.Padding(FConcertFrontendStyle::Get()->GetMargin("Concert.SessionRowPadding"))
		.OnAcceptFunc([this](const TSharedPtr<FConcertSessionTreeItem>& Item, const FString& ArchiveName) { RequestArchiveSession(Item, ArchiveName); }) // Accepting archive the session.
		.OnDeclineFunc([this](const TSharedPtr<FConcertSessionTreeItem>& Item) { RemoveSessionRow(Item); }) // Declining removes the editable 'save session as' row from the view.
		.HighlightText(this, &SConcertSessionBrowser::HighlightSearchText);
}

TSharedRef<ITableRow> SConcertSessionBrowser::MakeGroupRowWidget(const TSharedPtr<FConcertTreeItem>& Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	const bool bIsActive = Item == ActiveSessionsGroupItem;
	return SNew(SSessionGroupRow, OwnerTable)
		.Padding(FConcertFrontendStyle::Get()->GetMargin("Concert.SessionRowPadding"))
		.GroupType(bIsActive ? SSessionGroupRow::EGroupType::Active : SSessionGroupRow::EGroupType::Archived);
}

TSharedRef<ITableRow> SConcertSessionBrowser::MakeRestoreSessionRowWidget(const TSharedPtr<FConcertSessionTreeItem>& RestoreItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	// Add an editable 'Restore Session' row in the table to let the user enter a session name.
	return SNew(SSaveRestoreSessionRow, RestoreItem, OwnerTable)
		.Padding(FConcertFrontendStyle::Get()->GetMargin("Concert.SessionRowPadding"))
		.OnAcceptFunc([this](const TSharedPtr<FConcertSessionTreeItem>& Item, const FString& SessionName) { RequestRestoreSession(Item, SessionName); }) // Accepting restores the session.
		.OnDeclineFunc([this](const TSharedPtr<FConcertSessionTreeItem>& Item) { RemoveSessionRow(Item); }) // Declining removes the editable 'restore session as' row from the view.
		.HighlightText(this, &SConcertSessionBrowser::HighlightSearchText);
}

void SConcertSessionBrowser::InsertNewSessionEditableRowInternal()
{
	// Insert a 'new session' editable row.
	FConcertSessionTreeItem Item;
	Item.Type = FConcertSessionTreeItem::EType::NewSession;
	InsertEditableSessionRow(MakeShared<FConcertSessionTreeItem>(MoveTemp(Item)), nullptr);
}

void SConcertSessionBrowser::InsertRestoreSessionAsEditableRowInternal(const TSharedPtr<FConcertSessionTreeItem>& ArchivedItem)
{
	// Insert the 'restore session as ' editable row just below the 'archived' item to restore.
	InsertEditableSessionRow(MakeShared<FConcertSessionTreeItem>(ArchivedItem->MakeCopyAsType(FConcertSessionTreeItem::EType::RestoreSession)), ArchivedItem);
}

void SConcertSessionBrowser::InsertArchiveSessionAsEditableRow(const TSharedPtr<FConcertSessionTreeItem>& LiveItem)
{
	// Insert the 'save session as' editable row just below the 'active' item to save.
	InsertEditableSessionRow(MakeShared<FConcertSessionTreeItem>(LiveItem->MakeCopyAsType(FConcertSessionTreeItem::EType::SaveSession)), LiveItem);
}

void SConcertSessionBrowser::InsertEditableSessionRow(TSharedPtr<FConcertSessionTreeItem> EditableItem, TSharedPtr<FConcertSessionTreeItem> ParentItem)
{
	// Insert the new row below its parent (if any).
	const EInsertPosition InsertPosition = InsertSessionAfterParentIfAvailableOrAtBeginning(EditableItem, ParentItem);

	// Ensure there is only one editable row at the time, removing the row being edited (if any).
	RemoveSession(EditableSessionRow);
	EditableSessionRow = EditableItem;
	EditableSessionRowParent = ParentItem;

	// Ensure the editable row added is selected and visible.
	SessionsView->SetSelection(EditableItem);
	SessionsView->RequestListRefresh();

	// NOTE: Ideally, I would only use RequestScrollIntoView() to scroll the new item into view, but it did not work. If an item was added into an hidden part,
	//       it was not always scrolled correctly into view. RequestNavigateToItem() worked much better, except when inserting the very first row in the list, in
	//       such case calling the function would give the focus to the list view (showing a white dashed line around it).
	if (InsertPosition == EInsertPosition::AtBeginning)
	{
		SessionsView->ScrollToTop(); // Item is inserted at 0. (New session)
	}
	else
	{
		SessionsView->RequestNavigateToItem(EditableItem);
	}
}

void SConcertSessionBrowser::RemoveSessionRow(const TSharedPtr<FConcertSessionTreeItem>& Item)
{
	RemoveSession(Item);

	// Don't keep the editable row if its 'parent' is removed. (if the session to restore or archive gets deleted in the meantime)
	if (Item == EditableSessionRowParent)
	{
		RemoveSession(EditableSessionRow);
		EditableSessionRow.Reset();
	}

	// Clear the editable row state if its the one removed.
	if (Item == EditableSessionRow)
	{
		EditableSessionRow.Reset();
		EditableSessionRowParent.Reset();
	}

	SessionsView->RequestListRefresh();
}

TSharedRef<SWidget> SConcertSessionBrowser::MakeSessionTableFooter()
{
	return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock).Text_Lambda([this]()
			{
				// Don't count the 'New Session', 'Restore Session' and 'Archive Session' editable row, they are transient rows used for inline input only.
				const int32 DisplayedSessionNum = Sessions.Num() - (EditableSessionRow.IsValid() ? 1 : 0);
				const int32 AvailableSessionNum = GetController()->GetActiveSessions().Num() + GetController()->GetArchivedSessions().Num();
				const int32 ServerNum = GetController()->GetServers().Num();

				// If all discovered session are displayed (none excluded by a filter).
				if (DisplayedSessionNum == AvailableSessionNum)
				{
					if (GetController()->GetServers().Num() == 0)
					{
						return LOCTEXT("NoServerNoFilter", "No servers found");
					}
					else
					{
						return FText::Format(LOCTEXT("NSessionNServerNoFilter", "{0} {0}|plural(one=session,other=sessions) on {1} {1}|plural(one=server,other=servers)"),
							DisplayedSessionNum, ServerNum);
					}
				}
				else // A filter is excluding at least one session.
				{
					if (DisplayedSessionNum == 0)
					{
						return FText::Format(LOCTEXT("NoSessionMatchNServer", "No matching sessions ({0} total on {1} {1}|plural(one=server,other=servers))"),
							AvailableSessionNum, ServerNum);
					}
					else
					{
						return FText::Format(LOCTEXT("NSessionNServer", "Showing {0} of {1} {1}|plural(one=session,other=sessions) on {2} {2}|plural(one=server,other=servers)"),
							DisplayedSessionNum, AvailableSessionNum, ServerNum);
					}
				}
			})
		]

		+SHorizontalBox::Slot()
		.FillWidth(1.0)
		[
			SNew(SSpacer)
		];
}

void SConcertSessionBrowser::OnFilterMenuChecked(const FName MenuName)
{
	if (MenuName == ConcertBrowserUtils::LastModifiedCheckBoxMenuName)
	{
		switch (PersistentSettings->LastModifiedTimeFormat)
		{
		case ETimeFormat::Relative:
			PersistentSettings->LastModifiedTimeFormat = ETimeFormat::Absolute;
			break;
		case ETimeFormat::Absolute: 
			PersistentSettings->LastModifiedTimeFormat = ETimeFormat::Relative;
			break;
		default:
			checkNoEntry();
		}
	}
	else if (MenuName == ConcertBrowserUtils::ActiveSessionsCheckBoxMenuName)
	{
		PersistentSettings->bShowActiveSessions = !PersistentSettings->bShowActiveSessions;
	}
	else if (MenuName == ConcertBrowserUtils::ArchivedSessionsCheckBoxMenuName)
	{
		PersistentSettings->bShowArchivedSessions = !PersistentSettings->bShowArchivedSessions;
	}
	else if (MenuName == ConcertBrowserUtils::DefaultServerCheckBoxMenuName)
	{
		PersistentSettings->bShowDefaultServerSessionsOnly = !PersistentSettings->bShowDefaultServerSessionsOnly;
	}
	bRefreshSessionFilter = true;

	PersistentSettings->SaveConfig();
}

TSharedPtr<SWidget> SConcertSessionBrowser::MakeTableContextualMenu()
{
	if (SessionsView->GetSelectedItems().Num() == 0)
	{
		// If clicking the header row, show the options for hiding them
		return UE::ConcertSharedSlate::MakeTableContextMenu(SessionHeaderRow.ToSharedRef(), {}, true);
	}

	const TArray<TSharedPtr<FConcertSessionTreeItem>> SelectedItems = GetSelectedItems();
	if (SelectedItems.Num() == 0)
	{
		// If you click the groups then the context menu should not contain anything
		return nullptr;
	}
	TSharedPtr<FConcertSessionTreeItem> Item = SelectedItems[0];

	TSharedRef<FExtender> Extender = MakeShared<FExtender>();
	ExtendSessionContextMenu.ExecuteIfBound(Item, Extender.Get());
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr, Extender);

	// Section title.
	MenuBuilder.BeginSection(SessionContextMenuExtensionHooks::ManageSession, Item->Type == FConcertSessionTreeItem::EType::ActiveSession ?
		LOCTEXT("ActiveSessionSection", "Active Session"):
		LOCTEXT("ArchivedSessionSection", "Archived Session"));

	if (Item->Type == FConcertSessionTreeItem::EType::ActiveSession)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CtxMenuArchive", "Archive"),
			LOCTEXT("CtxMenuArchive_Tooltip", "Archived the Session"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this](){ OnArchiveButtonClicked(); }),
				FCanExecuteAction::CreateLambda([SelectedCount = SelectedItems.Num()] { return SelectedCount == 1; }),
				FIsActionChecked::CreateLambda([this] { return false; })),
			NAME_None,
			EUserInterfaceActionType::Button);
	}
	else // Archive
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CtxMenuRestore", "Restore"),
			LOCTEXT("CtxMenuRestore_Tooltip", "Restore the Session"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this](){ OnRestoreButtonClicked(); }),
				FCanExecuteAction::CreateLambda([SelectedCount = SelectedItems.Num()] { return SelectedCount == 1; }),
				FIsActionChecked::CreateLambda([this] { return false; })),
			NAME_None,
			EUserInterfaceActionType::Button);
	}

	MenuBuilder.AddMenuEntry(
		LOCTEXT("CtxMenuRename", "Rename"),
		LOCTEXT("CtxMenuRename_Tooltip", "Rename the Session"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this, Item](){ OnBeginEditingSessionName(Item); }),
			FCanExecuteAction::CreateLambda([this] { return IsRenameButtonEnabledInternal(); }),
			FIsActionChecked::CreateLambda([this] { return false; })),
		NAME_None,
		EUserInterfaceActionType::Button);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("CtxMenuDelete", "Delete"),
		FText::Format(LOCTEXT("CtxMenuDelete_Tooltip", "Delete the {0}|plural(one=Session,other=Sessions)"), SelectedItems.Num()),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this](){ OnDeleteButtonClicked(); }),
			FCanExecuteAction::CreateLambda([this] { return IsDeleteButtonEnabledInternal(); }),
			FIsActionChecked::CreateLambda([this] { return false; })),
		NAME_None,
		EUserInterfaceActionType::Button);

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SConcertSessionBrowser::MakeViewOptionsComboButtonMenu()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("DisplayLastModifiedInRelativeTime", "Display Relative Time"),
		TAttribute<FText>::CreateLambda([this]()
		{
			const bool bIsVisible = SessionHeaderRow->IsColumnVisible(ConcertBrowserUtils::LastModifiedColName);
			return bIsVisible
				? LOCTEXT("DisplayLastModifiedInRelativeTime.Tooltip.Visible", "Display the Last Modified column in relative time?")
				: LOCTEXT("DisplayLastModifiedInRelativeTime.Tooltip.Hidden", "Disabled because the Last Modified column is hidden.");
		}),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SConcertSessionBrowser::OnFilterMenuChecked, ConcertBrowserUtils::LastModifiedCheckBoxMenuName),
			FCanExecuteAction::CreateLambda([this] { return SessionHeaderRow->IsColumnVisible(ConcertBrowserUtils::LastModifiedColName); }),
			FIsActionChecked::CreateLambda([this] { return PersistentSettings->LastModifiedTimeFormat == ETimeFormat::Relative; })),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);
	MenuBuilder.AddSeparator();
	
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ActiveSessions_Label", "Active Sessions"),
		LOCTEXT("ActiveSessions_Tooltip", "Displays Active Sessions"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SConcertSessionBrowser::OnFilterMenuChecked, ConcertBrowserUtils::ActiveSessionsCheckBoxMenuName),
			FCanExecuteAction::CreateLambda([] { return true; }),
			FIsActionChecked::CreateLambda([this] { return PersistentSettings->bShowActiveSessions; })),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ArchivedSessions_Label", "Archived Sessions"),
		LOCTEXT("ArchivedSessions_Tooltip", "Displays Archived Sessions"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SConcertSessionBrowser::OnFilterMenuChecked, ConcertBrowserUtils::ArchivedSessionsCheckBoxMenuName),
			FCanExecuteAction::CreateLambda([] { return true; }),
			FIsActionChecked::CreateLambda([this] { return PersistentSettings->bShowArchivedSessions; })),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("DefaultServer_Label", "Default Server Sessions"),
		LOCTEXT("DefaultServer_Tooltip", "Displays Sessions Hosted By the Default Server"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SConcertSessionBrowser::OnFilterMenuChecked, ConcertBrowserUtils::DefaultServerCheckBoxMenuName),
			FCanExecuteAction::CreateLambda([] { return true; }),
			FIsActionChecked::CreateLambda([this] { return PersistentSettings->bShowDefaultServerSessionsOnly; })),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	return MenuBuilder.MakeWidget();
}

void SConcertSessionBrowser::OnSessionSelectionChanged(TSharedPtr<FConcertTreeItem> SelectedSession, ESelectInfo::Type SelectInfo)
{
	if (!IsGroupItem(SelectedSession))
	{
		OnSessionSelectionChanged(StaticCastSharedPtr<FConcertSessionTreeItem>(SelectedSession), SelectInfo);
	}
}

void SConcertSessionBrowser::OnSessionSelectionChanged(TSharedPtr<FConcertSessionTreeItem> SelectedSession, ESelectInfo::Type SelectInfo)
{
	// Cancel editing the row to create, archive or restore a session (if any), unless the row was selected in code.
	if (EditableSessionRow.IsValid() && SelectInfo != ESelectInfo::Direct)
	{
		RemoveSessionRow(EditableSessionRow);
		check(!EditableSessionRow.IsValid() && !EditableSessionRowParent.IsValid()); // Expect to be cleared by RemoveSessionRow().
	}

	// Clear the list of clients (if any)
	Clients.Reset();

	OnSessionClicked.ExecuteIfBound(SelectedSession);
}

bool SConcertSessionBrowser::IsNewButtonEnabledInternal() const
{
	return GetController()->CanEverCreateSessions() && GetController()->GetServers().Num() > 0;
}

bool SConcertSessionBrowser::IsRestoreButtonEnabledInternal() const
{
	TArray<TSharedPtr<FConcertSessionTreeItem>> SelectedItems = GetSelectedItems();
	return SelectedItems.Num() == 1 && SelectedItems[0]->Type == FConcertSessionTreeItem::EType::ArchivedSession;
}

bool SConcertSessionBrowser::IsArchiveButtonEnabledInternal() const
{
	TArray<TSharedPtr<FConcertSessionTreeItem>> SelectedItems = GetSelectedItems();
	return SelectedItems.Num() == 1 && SelectedItems[0]->Type == FConcertSessionTreeItem::EType::ActiveSession;
}

bool SConcertSessionBrowser::IsRenameButtonEnabledInternal() const
{
	TArray<TSharedPtr<FConcertSessionTreeItem>> SelectedItems = GetSelectedItems();
	if (SelectedItems.Num() != 1)
	{
		return false;
	}

	return (SelectedItems[0]->Type == FConcertSessionTreeItem::EType::ActiveSession && GetController()->CanRenameActiveSession(SelectedItems[0]->ServerAdminEndpointId, SelectedItems[0]->SessionId)) ||
	       (SelectedItems[0]->Type == FConcertSessionTreeItem::EType::ArchivedSession && GetController()->CanRenameArchivedSession(SelectedItems[0]->ServerAdminEndpointId, SelectedItems[0]->SessionId));
}

bool SConcertSessionBrowser::IsDeleteButtonEnabledInternal() const
{
	TArray<TSharedPtr<FConcertSessionTreeItem>> SelectedItems = GetSelectedItems();
	if (SelectedItems.Num() == 0)
	{
		return false;
	}

	return (SelectedItems[0]->Type == FConcertSessionTreeItem::EType::ActiveSession && GetController()->CanDeleteActiveSession(SelectedItems[0]->ServerAdminEndpointId, SelectedItems[0]->SessionId)) ||
	       (SelectedItems[0]->Type == FConcertSessionTreeItem::EType::ArchivedSession && GetController()->CanDeleteArchivedSession(SelectedItems[0]->ServerAdminEndpointId, SelectedItems[0]->SessionId));
}

FReply SConcertSessionBrowser::OnNewButtonClicked()
{
	InsertNewSessionEditableRowInternal();
	return FReply::Handled();
}

FReply SConcertSessionBrowser::OnRestoreButtonClicked()
{
	const TArray<TSharedPtr<FConcertSessionTreeItem>> SelectedItems = GetSelectedItems();
	if (SelectedItems.Num() == 1)
	{
		InsertRestoreSessionAsEditableRowInternal(SelectedItems[0]);
	}

	return FReply::Handled();
}

FReply SConcertSessionBrowser::OnArchiveButtonClicked()
{
	const TArray<TSharedPtr<FConcertSessionTreeItem>> SelectedItems = GetSelectedItems();
	if (SelectedItems.Num() == 1)
	{
		InsertArchiveSessionAsEditableRow(SelectedItems[0]);
	}

	return FReply::Handled();
}

FReply SConcertSessionBrowser::OnDeleteButtonClicked()
{
	RequestDeleteSessions(GetSelectedItems());
	return FReply::Handled();
}

void SConcertSessionBrowser::OnBeginEditingSessionName(TSharedPtr<FConcertSessionTreeItem> Item)
{
	Item->OnBeginEditSessionNameRequest.Broadcast(); // Signal the row widget to enter in edit mode.
}

void SConcertSessionBrowser::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Ensure the 'default server' filter is updated when the configuration of the default server changes.
	if (DefaultServerUrl.IsSet() && LastDefaultServerUrl != DefaultServerUrl.Get())
	{
		LastDefaultServerUrl = DefaultServerUrl.Get();
		bRefreshSessionFilter = true;
	}

	// Should refresh the session filter?
	if (bRefreshSessionFilter)
	{
		RefreshSessionList();
		bRefreshSessionFilter = false;
	}
}

FReply SConcertSessionBrowser::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	// NOTE: When an 'editable row' text box has the focus the keys are grabbed by the text box but
	//       if the editable row is still selected, but the text field doesn't have the focus anymore
	//       the keys will end up here if the browser has the focus.

	if (InKeyEvent.GetKey() == EKeys::Delete && !EditableSessionRow.IsValid()) // Delete selected row(s) unless the selected row is an 'editable' one.
	{
		RequestDeleteSessions(GetSelectedItems());
		return FReply::Handled();
	}
	
	if (InKeyEvent.GetKey() == EKeys::Escape && EditableSessionRow.IsValid()) // Cancel 'new session', 'archive session' or 'restore session' action.
	{
		RemoveSessionRow(EditableSessionRow);
		check(!EditableSessionRow.IsValid() && !EditableSessionRowParent.IsValid()); // Expect to be cleared by RemoveSessionRow().
		return FReply::Handled();
	}
	
	if (InKeyEvent.GetKey() == EKeys::F2 && !EditableSessionRow.IsValid())
	{
		TArray<TSharedPtr<FConcertSessionTreeItem>> SelectedItems = GetSelectedItems();
		if (SelectedItems.Num() == 1)
		{
			SelectedItems[0]->OnBeginEditSessionNameRequest.Broadcast(); // Broadcast the request.
		}
	}

	return FReply::Unhandled();
}

TArray<TSharedPtr<FConcertSessionTreeItem>> SConcertSessionBrowser::GetSelectedItems() const
{
	TArray<TSharedPtr<FConcertSessionTreeItem>> Result;
	const TArray<TSharedPtr<FConcertTreeItem>> Selected = SessionsView->GetSelectedItems();
	Algo::ForEachIf(
		Selected,
		[this](const TSharedPtr<FConcertTreeItem>& Item){ return !IsGroupItem(Item); },
		[&Result](const TSharedPtr<FConcertTreeItem>& Item){ Result.Add(StaticCastSharedPtr<FConcertSessionTreeItem>(Item)); }
		);
	return Result;
}

void SConcertSessionBrowser::RequestCreateSession(const TSharedPtr<FConcertSessionTreeItem>& NewItem)
{
	GetController()->CreateSession(NewItem->ServerAdminEndpointId, NewItem->SessionName, NewItem->ProjectName);
	RemoveSessionRow(NewItem); // The row used to edit the session name and pick the server.
}

void SConcertSessionBrowser::RequestArchiveSession(const TSharedPtr<FConcertSessionTreeItem>& SaveItem, const FString& ArchiveName)
{
	GetController()->ArchiveSession(SaveItem->ServerAdminEndpointId, SaveItem->SessionId, ArchiveName, FConcertSessionFilter());
	RemoveSessionRow(SaveItem); // The row used to edit the archive name.
}

void SConcertSessionBrowser::RequestRestoreSession(const TSharedPtr<FConcertSessionTreeItem>& RestoreItem, const FString& SessionName)
{
	GetController()->RestoreSession(RestoreItem->ServerAdminEndpointId, RestoreItem->SessionId, SessionName, FConcertSessionFilter());
	RemoveSessionRow(RestoreItem); // The row used to edit the restore as name.
}

void SConcertSessionBrowser::RequestRenameSession(const TSharedPtr<FConcertSessionTreeItem>& RenamedItem, const FString& NewName)
{
	if (RenamedItem->Type == FConcertSessionTreeItem::EType::ActiveSession)
	{
		GetController()->RenameActiveSession(RenamedItem->ServerAdminEndpointId, RenamedItem->SessionId, NewName);
	}
	else if (RenamedItem->Type == FConcertSessionTreeItem::EType::ArchivedSession)
	{
		GetController()->RenameArchivedSession(RenamedItem->ServerAdminEndpointId, RenamedItem->SessionId, NewName);
	}

	// Display the new name until the server response is received. If the server refuses the new name, the discovery will reset the
	// name (like if another client renamed it back) and the user will get a toast saying the rename failed.
	RenamedItem->SessionName = NewName;
}

void SConcertSessionBrowser::RequestDeleteSessions(const TArray<TSharedPtr<FConcertSessionTreeItem>>& ItemsToDelete)
{
	if (ItemsToDelete.Num() == 0)
	{
		return;
	}
	
	if (!AskUserToDeleteSessions.IsBound() || AskUserToDeleteSessions.Execute(ItemsToDelete))
	{
		ConcertBrowserUtils::RequestItemDeletion(*GetController().Get(), ItemsToDelete);
	}
	PostRequestedDeleteSession.ExecuteIfBound(ItemsToDelete);
}

#undef LOCTEXT_NAMESPACE
