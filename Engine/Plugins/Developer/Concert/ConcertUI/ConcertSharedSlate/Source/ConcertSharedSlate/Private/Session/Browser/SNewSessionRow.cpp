// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNewSessionRow.h"

#include "ConcertFrontendStyle.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Session/Browser/ConcertBrowserUtils.h"



#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"

#define LOCTEXT_NAMESPACE "SConcertBrowser"

void SNewSessionRow::Construct(const FArguments& InArgs, TSharedPtr<FConcertSessionTreeItem> InItem, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	Item = MoveTemp(InItem);
	GetServersFunc = InArgs._GetServerFunc;
	AcceptFunc = InArgs._OnAcceptFunc;
	DeclineFunc = InArgs._OnDeclineFunc;
	HighlightText = InArgs._HighlightText;
	DefaultServerURL = InArgs._DefaultServerURL;

	check(GetServersFunc);
	check(AcceptFunc);
	check(DeclineFunc);

	// Construct base class
	SMultiColumnTableRow<TSharedPtr<FConcertSessionTreeItem>>::Construct(FSuperRowType::FArguments().Padding(InArgs._Padding), InOwnerTableView);
	
	// This rows are needed to correctly display the row
	TemporaryColumnShower
		.SetHeaderRow(InOwnerTableView->GetHeaderRow())
		.SaveVisibilityAndSet(ConcertBrowserUtils::SessionColName, true)
		.SaveVisibilityAndSet(ConcertBrowserUtils::ServerColName, true)
		.SaveVisibilityAndSet(ConcertBrowserUtils::VersionColName, true)
		.SaveVisibilityAndSet(ConcertBrowserUtils::ProjectColName, true);

	// Fill the server combo.
	UpdateServerList();
}

SNewSessionRow::~SNewSessionRow()
{
	TemporaryColumnShower.ResetToSavedVisibilities();
}

void SNewSessionRow::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Performance could be improved by only updating it when the server list has changed but it's no biggy...
	UpdateServerList();

	// Should give the focus to an editable text.
	if (!bInitialFocusTaken)
	{
		bInitialFocusTaken = FSlateApplication::Get().SetKeyboardFocus(EditableSessionName.ToSharedRef());
	}
}

TSharedRef<SWidget> SNewSessionRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	TSharedPtr<FConcertSessionTreeItem> ItemPin = Item.Pin();

	if (ColumnName == ConcertBrowserUtils::SessionColName)
	{
		return SNew( SHorizontalBox )

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(6, 0, 0, 0)
			[
				SNew( SExpanderArrow, SharedThis(this) ).IndentAmount(12)
			]

			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				.Padding(FMargin(0, 0, 2, 0))
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Padding(1.f, 1.f, 4.f, 1.f)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
					]
					
					+SHorizontalBox::Slot()
					[
						SAssignNew(EditableSessionName, SEditableTextBox)
						.HintText(LOCTEXT("EnterSessionNameHint", "Enter a session name"))
						.OnTextCommitted(this, &SNewSessionRow::OnSessionNameCommitted)
						.OnKeyDownHandler(this, &SNewSessionRow::OnKeyDownHandler)
						.OnTextChanged(this, &SNewSessionRow::OnSessionNameChanged)
					]
				]
			];
	}
	if (ColumnName == ConcertBrowserUtils::ServerColName)
	{
		return SNew(SHorizontalBox)

			// 'Server' combo
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 1)
			[
				SAssignNew(ServersComboBox, SComboBox<TSharedPtr<FConcertServerInfo>>)
				.OptionsSource(&Servers)
				.OnGenerateWidget(this, &SNewSessionRow::OnGenerateServersComboOptionWidget)
				.ToolTipText(LOCTEXT("SelectServerTooltip", "Select the server on which the session should be created"))
				[
					MakeSelectedServerWidget()
				]
			];
	}
	if (ColumnName == ConcertBrowserUtils::VersionColName)
	{
		return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f)
			.HAlign(HAlign_Left)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FMargin(1.0f, 0.0f))

				// 'Accept' button
				+SUniformGridPanel::Slot(0, 0)
				[
					ConcertBrowserUtils::MakePositiveActionButton(
						FAppStyle::GetBrush("Icons.Check"),
						LOCTEXT("CreateCheckIconTooltip", "Create the session"),
						TAttribute<bool>::Create([this]() { return !EditableSessionName->GetText().IsEmpty(); }),
						FOnClicked::CreateRaw(this, &SNewSessionRow::OnAccept))
				]

				// 'Decline' button
				+SUniformGridPanel::Slot(1, 0)
				[
					ConcertBrowserUtils::MakeNegativeActionButton(
						FAppStyle::GetBrush("Icons.X"),
						LOCTEXT("CancelIconTooltip", "Cancel"),
						true, // Always enabled.
						FOnClicked::CreateRaw(this, &SNewSessionRow::OnDecline))
				]
			];
	}
	if (ColumnName == ConcertBrowserUtils::ProjectColName)
	{
		FText ProjectName = FText::AsCultureInvariant(FApp::GetProjectName());
		return SNew(SBox)
			.VAlign(VAlign_Center)
			.Padding(FMargin(0, 0, 2, 0))
			[
				SAssignNew(ProjectNameTextBox, SEditableTextBox)
				.HintText(ProjectName)
				.Text(ProjectName)
				.OnTextCommitted(this, &SNewSessionRow::OnProjectNameCommitted)
				.OnKeyDownHandler(this, &SNewSessionRow::OnKeyDownHandler)
			];
	}
	else
	{
		check(ColumnName == ConcertBrowserUtils::LastModifiedColName);
		return SNullWidget::NullWidget;
	}
}

TSharedRef<SWidget> SNewSessionRow::OnGenerateServersComboOptionWidget(TSharedPtr<FConcertServerInfo> ServerItem)
{
	bool bIsDefaultServer = ServerItem->ServerName == DefaultServerURL.Get();

	FText Tooltip;
	if (bIsDefaultServer)
	{
		Tooltip = LOCTEXT("DefaultServerTooltip", "Default Configured Server");
	}
	else if (ServerItem->ServerName == FPlatformProcess::ComputerName())
	{
		Tooltip = LOCTEXT("LocalServerTooltip", "Local Server Running on This Computer");
	}
	else
	{
		Tooltip = LOCTEXT("OnlineServerTooltip", "Online Server");
	}

	return SNew(SHorizontalBox)
		.ToolTipText(Tooltip)

		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Font(bIsDefaultServer ? FAppStyle::Get().GetFontStyle("BoldFont") : FAppStyle::Get().GetFontStyle("NormalFont"))
			.Text(GetServerDisplayName(ServerItem->ServerName))
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			ConcertBrowserUtils::MakeServerVersionIgnoredWidget(ServerItem->ServerFlags)
		];
}

void SNewSessionRow::UpdateServerList()
{
	// Remember the currently selected item (if any).
	TSharedPtr<FConcertServerInfo> SelectedItem = ServersComboBox->GetSelectedItem(); // Instance in current list.

	// Clear the current list. The list is rebuilt from scratch.
	Servers.Reset();

	TSharedPtr<FConcertServerInfo> LocalServerInfo;
	TSharedPtr<FConcertServerInfo> DefaultServerInfo;
	TSharedPtr<FConcertServerInfo> SelectedServerInfo; // Instance in the new list.

	// Convert to shared ptr (slate needs that) and find if the latest list contains a default/local server.
	for (const FConcertServerInfo& ServerInfo : GetServersFunc())
	{
		TSharedPtr<FConcertServerInfo> ComboItem = MakeShared<FConcertServerInfo>(ServerInfo);
		
		// Default server is deemed more important than local server to display the icon aside the server.
		if (ComboItem->ServerName == DefaultServerURL.Get()) 
		{
			DefaultServerInfo = ComboItem;
		}
		else if (ComboItem->ServerName == FPlatformProcess::ComputerName())
		{
			LocalServerInfo = ComboItem;
		}

		if (SelectedItem.IsValid() && SelectedItem->ServerName == ComboItem->ServerName)
		{
			SelectedServerInfo = ComboItem; // Preserve user selection using the new instance.
		}

		Servers.Emplace(MoveTemp(ComboItem));
	}

	// Sort the server list alphabetically.
	Servers.Sort([](const TSharedPtr<FConcertServerInfo>& Lhs, const TSharedPtr<FConcertServerInfo>& Rhs) { return Lhs->ServerName < Rhs->ServerName; });

	// If a server is running on this machine, put it first in the list.
	if (LocalServerInfo.IsValid() && Servers[0] != LocalServerInfo)
	{
		Servers.Remove(LocalServerInfo); // Keep sort order.
		Servers.Insert(LocalServerInfo, 0);
	}

	// If a 'default server' is configured and available, put it first in the list. (Possibly overruling the local one)
	if (DefaultServerInfo.IsValid() && Servers[0] != DefaultServerInfo)
	{
		Servers.Remove(DefaultServerInfo); // Keep sort order.
		Servers.Insert(DefaultServerInfo, 0);
	}

	// If a server was selected and is still in the updated list.
	if (SelectedServerInfo.IsValid())
	{
		// Preserve user selection.
		ServersComboBox->SetSelectedItem(SelectedServerInfo);
	}
	else if (Servers.Num())
	{
		// Select the very first item in the list which is most likely be the default or the local server as they were put first above.
		ServersComboBox->SetSelectedItem(Servers[0]);
	}
	else // Server list is empty.
	{
		ServersComboBox->ClearSelection();
		Servers.Reset();
	}

	ServersComboBox->RefreshOptions();
}

TSharedRef<SWidget> SNewSessionRow::MakeSelectedServerWidget()
{
	return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(this, &SNewSessionRow::GetSelectedServerText)
			.HighlightText(HighlightText)
			.Font(FCoreStyle::GetDefaultFontStyle("Regular", FConcertFrontendStyle::Get()->GetFloat("Concert.SessionBrowser.FontSize")))
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2, 0, 0, 0)
		[
			SNew(SImage)
			.Image(this, &SNewSessionRow::GetSelectedServerIgnoreVersionImage)
			.ToolTipText(this, &SNewSessionRow::GetSelectedServerIgnoreVersionTooltip)
		];
}

FText SNewSessionRow::GetSelectedServerText() const
{
	TSharedPtr<FConcertServerInfo> SelectedServer = ServersComboBox->GetSelectedItem();
	if (SelectedServer.IsValid())
	{
		return GetServerDisplayName(SelectedServer->ServerName);
	}
	return LOCTEXT("SelectAServer", "Select a Server");
}

FText SNewSessionRow::GetServerDisplayName(const FString& ServerName) const
{
	const bool bIsDefaultServer = ServerName == DefaultServerURL.Get();
	if (bIsDefaultServer)
	{
		return FText::Format(LOCTEXT("DefaultServer", "{0} (Default)"), FText::FromString(FPlatformProcess::ComputerName()));
	}
	if (ServerName == FPlatformProcess::ComputerName())
	{
		return FText::Format(LOCTEXT("MyComputer", "{0} (My Computer)"), FText::FromString(FPlatformProcess::ComputerName()));
	}
	return FText::FromString(ServerName);
}

const FSlateBrush* SNewSessionRow::GetSelectedServerIgnoreVersionImage() const
{
	if (ServersComboBox->GetSelectedItem() && (ServersComboBox->GetSelectedItem()->ServerFlags & EConcertServerFlags::IgnoreSessionRequirement) != EConcertServerFlags::None)
	{
		return FAppStyle::GetBrush("Icons.Warning");
	}
	return FAppStyle::GetNoBrush();
}

FText SNewSessionRow::GetSelectedServerIgnoreVersionTooltip() const
{
	if (ServersComboBox->GetSelectedItem() && (ServersComboBox->GetSelectedItem()->ServerFlags & EConcertServerFlags::IgnoreSessionRequirement) != EConcertServerFlags::None)
	{
		return ConcertBrowserUtils::GetServerVersionIgnoredTooltip();
	}
	return FText();
}

FReply SNewSessionRow::OnAccept()
{
	TSharedPtr<FConcertSessionTreeItem> ItemPin = Item.Pin();
	FString NewSessionName = EditableSessionName->GetText().ToString();
	FText InvalidNameErrorMsg = ConcertSettingsUtils::ValidateSessionName(NewSessionName);

	if (InvalidNameErrorMsg.IsEmpty())
	{
		ItemPin->SessionName = NewSessionName;
		ItemPin->ServerName = ServersComboBox->GetSelectedItem()->ServerName;
		ItemPin->ServerAdminEndpointId = ServersComboBox->GetSelectedItem()->AdminEndpointId;
		FString CandidateProjectName = ProjectNameTextBox->GetText().ToString();
		if (CandidateProjectName.IsEmpty())
		{
			CandidateProjectName = FApp::GetProjectName();
		}
		ItemPin->ProjectName = CandidateProjectName;
		AcceptFunc(ItemPin); // Delegate to create the session.
	}
	else
	{
		EditableSessionName->SetError(InvalidNameErrorMsg);
		FSlateApplication::Get().SetKeyboardFocus(EditableSessionName);
	}

	return FReply::Handled();
}

FReply SNewSessionRow::OnDecline()
{
	DeclineFunc(Item.Pin()); // Decline the creation and remove the row from the table.
	return FReply::Handled();
}

void SNewSessionRow::OnSessionNameChanged(const FText& NewName)
{
	EditableSessionName->SetError(ConcertSettingsUtils::ValidateSessionName(NewName.ToString()));
}

void SNewSessionRow::OnProjectNameCommitted(const FText &NewProjectName, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnEnter)
	{
		OnAccept(); // Create the session;.
	}
}

void SNewSessionRow::OnSessionNameCommitted(const FText& NewText, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnEnter)
	{
		OnAccept(); // Create the session.
	}
}

FReply SNewSessionRow::OnKeyDownHandler(const FGeometry&, const FKeyEvent& KeyEvent)
{
	// NOTE: This is invoked when the editable text field has the focus.
	return KeyEvent.GetKey() == EKeys::Escape ? OnDecline() : FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
