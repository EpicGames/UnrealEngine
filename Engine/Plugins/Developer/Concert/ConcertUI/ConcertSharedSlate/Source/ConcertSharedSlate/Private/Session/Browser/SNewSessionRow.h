// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertHeaderRowUtils.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class STableViewBase;
struct FConcertServerInfo;
template <typename OptionType> class SComboBox;

class FConcertSessionTreeItem;
class SEditableTextBox;
class SHeaderRow;
class SWidget;

/**
 * The type of row used in the session list view to edit a new session (the session name + server).
 */
class SNewSessionRow : public SMultiColumnTableRow<TSharedPtr<FConcertSessionTreeItem>>
{
public:
	using FGetServersFunc = TFunction<TArray<FConcertServerInfo>()>;
	// Should remove the editable 'new' row and creates the sessions.
	using FAcceptFunc = TFunction<void(const TSharedPtr<FConcertSessionTreeItem>&)>;
	// Should just remove the editable 'new' row.
	using FDeclineFunc = TFunction<void(const TSharedPtr<FConcertSessionTreeItem>&)>; 

	SLATE_BEGIN_ARGS(SNewSessionRow) // Needed to use Args because Construct() is limited to 5 arguments and 6 were required.
		: _GetServerFunc()
		, _OnAcceptFunc()
		, _OnDeclineFunc()
		, _HighlightText()
	{}
		SLATE_ARGUMENT(FGetServersFunc, GetServerFunc)
		SLATE_ARGUMENT(FAcceptFunc, OnAcceptFunc)
		SLATE_ARGUMENT(FDeclineFunc, OnDeclineFunc)

		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_ATTRIBUTE(FString, DefaultServerURL)
	
		SLATE_ATTRIBUTE(FMargin, Padding)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, TSharedPtr<FConcertSessionTreeItem> InItem, const TSharedRef<STableViewBase>& InOwnerTableView);
	virtual ~SNewSessionRow() override;
	
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:

	TSharedRef<SWidget> OnGenerateServersComboOptionWidget(TSharedPtr<FConcertServerInfo> Item);
	TSharedRef<SWidget> MakeSelectedServerWidget();
	FText GetSelectedServerText() const;
	const FSlateBrush* GetSelectedServerIgnoreVersionImage() const;
	FText GetSelectedServerIgnoreVersionTooltip() const;
	FText GetServerDisplayName(const FString& ServerName) const;

	void OnSessionNameChanged(const FText& NewName);
	void OnSessionNameCommitted(const FText& NewText, ETextCommit::Type CommitType);
	void OnProjectNameCommitted(const FText& NewProjectName, ETextCommit::Type CommitType);

	FReply OnAccept();
	FReply OnDecline();
	FReply OnKeyDownHandler(const FGeometry&, const FKeyEvent&); // Registered as handler to the editable text (vs. OnKeyDown() virtual method).

	void UpdateServerList();

	/** Helper for temporarily showing all columns required for this column */
	UE::ConcertSharedSlate::FColumnVisibilityTransaction TemporaryColumnShower;
	
	/** Holds the new item to fill with session name and server. */
	TWeakPtr<FConcertSessionTreeItem> Item;
	/** Servers displayed in the server combo box. */
	TArray<TSharedPtr<FConcertServerInfo>> Servers;

	TSharedPtr<SComboBox<TSharedPtr<FConcertServerInfo>>> ServersComboBox;
	TSharedPtr<SEditableTextBox> EditableSessionName;
	TSharedPtr<SEditableTextBox> ProjectNameTextBox;

	FGetServersFunc GetServersFunc;
	FAcceptFunc AcceptFunc;
	FDeclineFunc DeclineFunc;

	TAttribute<FText> HighlightText;
	TAttribute<FString> DefaultServerURL;
	bool bInitialFocusTaken = false;
};
