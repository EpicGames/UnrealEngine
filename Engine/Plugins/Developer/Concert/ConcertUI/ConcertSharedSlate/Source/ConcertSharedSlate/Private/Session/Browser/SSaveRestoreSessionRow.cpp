// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSaveRestoreSessionRow.h"

#include "ConcertFrontendStyle.h"
#include "Session/Browser/ConcertBrowserUtils.h"



#include "Framework/Application/SlateApplication.h"
#include "Internationalization/Regex.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SConcertBrowser"

void SSaveRestoreSessionRow::Construct(const FArguments& InArgs, TSharedPtr<FConcertSessionTreeItem> InNode, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	Item = MoveTemp(InNode);
	AcceptFunc = InArgs._OnAcceptFunc;
	DeclineFunc = InArgs._OnDeclineFunc;
	HighlightText = InArgs._HighlightText;

	check(AcceptFunc);
	check(DeclineFunc);

	// Construct base class
	SMultiColumnTableRow<TSharedPtr<FConcertSessionTreeItem>>::Construct(FSuperRowType::FArguments().Padding(InArgs._Padding), InOwnerTableView);

	// This rows are needed to correctly display the row
	TemporaryColumnShower
		.SetHeaderRow(InOwnerTableView->GetHeaderRow())
		.SaveVisibilityAndSet(ConcertBrowserUtils::SessionColName, true)
		.SaveVisibilityAndSet(ConcertBrowserUtils::ServerColName, true)
		.SaveVisibilityAndSet(ConcertBrowserUtils::ProjectColName, true);
}

SSaveRestoreSessionRow::~SSaveRestoreSessionRow()
{
	TemporaryColumnShower.ResetToSavedVisibilities();
}

TBitArray<> SSaveRestoreSessionRow::GetWiresNeededByDepth() const
{
	TBitArray<> Bits;
	Bits.Add(false);
	return Bits;
}

FText SSaveRestoreSessionRow::GetDefaultName(const FConcertSessionTreeItem& InItem) const
{
	if (InItem.Type == FConcertSessionTreeItem::EType::SaveSession)
	{
		return FText::Format(LOCTEXT("DefaultName", "{0}.{1}"), FText::FromString(InItem.SessionName), FText::FromString(FDateTime::UtcNow().ToString()));
	}

	// Supposing the name of the archive has the dates as suffix, like SessionXYZ.2019.03.13-19.39.12, then extracts SessionXYZ
	static FRegexPattern Pattern(TEXT(R"((.*)\.\d+\.\d+\.\d+\-\d+\.\d+\.\d+$)"));
	FRegexMatcher Matcher(Pattern, InItem.SessionName);
	if (Matcher.FindNext())
	{
		return FText::FromString(Matcher.GetCaptureGroup(1));
	}

	return FText::FromString(InItem.SessionName);
}

TSharedRef<SWidget> SSaveRestoreSessionRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	TSharedPtr<FConcertSessionTreeItem> ItemPin = Item.Pin();

	if (ColumnName == ConcertBrowserUtils::SessionColName)
	{
		return SNew( SHorizontalBox )

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(6, 0, 0, 0)
			[
				SNew( SExpanderArrow, SharedThis(this) ).IndentAmount(12)
			]

			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(8, 0)
				[
					SNew(SExpanderArrow, SharedThis(this))
					.IndentAmount(4)
					.StyleSet(&FAppStyle::Get())
					.ShouldDrawWires(true)
				]
				
				// 'Restore as/Save as' text
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(2.0, 0.0)
				[
					SNew(STextBlock)
					.Text(ItemPin->Type == FConcertSessionTreeItem::EType::RestoreSession ? LOCTEXT("RestoreAs", "Restore as:") : LOCTEXT("ArchiveAs", "Archive as:"))
					.Font(FCoreStyle::GetDefaultFontStyle("Regular", FConcertFrontendStyle::Get()->GetFloat("Concert.SessionBrowser.FontSize")))
				]

				// Editable text.
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0, 0, 2, 0))
				[
					SAssignNew(EditableSessionName, SEditableTextBox)
					.HintText(ItemPin->Type == FConcertSessionTreeItem::EType::RestoreSession ? LOCTEXT("RestoreSessionHint", "Enter a session name") : LOCTEXT("ArchivSessionHint", "Enter an archive name"))
					.OnTextCommitted(this, &SSaveRestoreSessionRow::OnSessionNameCommitted)
					.OnKeyDownHandler(this, &SSaveRestoreSessionRow::OnKeyDownHandler)
					.OnTextChanged(this, &SSaveRestoreSessionRow::OnSessionNameChanged)
					.Text(GetDefaultName(*ItemPin))
					.SelectAllTextWhenFocused(true)
					.Font(FCoreStyle::GetDefaultFontStyle("Regular", FConcertFrontendStyle::Get()->GetFloat("Concert.SessionBrowser.FontSize")))
				]
			];
	}
	if (ColumnName == ConcertBrowserUtils::ServerColName)
	{
		// 'Server' text block.
		return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.VAlign(VAlign_Center)
			[
				// Server.
				SNew(STextBlock)
				.Text(FText::FromString(ItemPin->ServerName))
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", FConcertFrontendStyle::Get()->GetFloat("Concert.SessionBrowser.FontSize")))
				.HighlightText(HighlightText)
			]
		];
	}
	if (ColumnName == ConcertBrowserUtils::ProjectColName)
	{
		return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f)
			.HAlign(HAlign_Left)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FMargin(1.0f, 0.0f))

				// 'Accept' button
				+SUniformGridPanel::Slot(0, 0)
				[
					ConcertBrowserUtils::MakePositiveActionButton(
						FAppStyle::GetBrush("Icons.Check"),
						ItemPin->Type == FConcertSessionTreeItem::EType::RestoreSession ? LOCTEXT("RestoreCheckIconTooltip", "Restore the session") : LOCTEXT("ArchiveCheckIconTooltip", "Archive the session"),
						TAttribute<bool>::Create([this]() { return !EditableSessionName->GetText().IsEmpty(); }), // Enabled?
						FOnClicked::CreateRaw(this, &SSaveRestoreSessionRow::OnAccept))
				]

				// 'Cancel' button
				+SUniformGridPanel::Slot(1, 0)
				[
					ConcertBrowserUtils::MakeNegativeActionButton(
						FAppStyle::GetBrush("Icons.X"),
						LOCTEXT("CancelTooltip", "Cancel"),
						true, // Enabled?
						FOnClicked::CreateRaw(this, &SSaveRestoreSessionRow::OnDecline))
				]
			];
	}
	else
	{
		check(ColumnName == ConcertBrowserUtils::VersionColName || ColumnName == ConcertBrowserUtils::LastModifiedColName);
		return SNullWidget::NullWidget;
	}
}

void SSaveRestoreSessionRow::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Should give the focus to an editable text.
	if (!bInitialFocusTaken)
	{
		bInitialFocusTaken = FSlateApplication::Get().SetKeyboardFocus(EditableSessionName.ToSharedRef());
	}
}

void SSaveRestoreSessionRow::OnSessionNameChanged(const FText& NewName)
{
	EditableSessionName->SetError(ConcertSettingsUtils::ValidateSessionName(NewName.ToString()));
}

void SSaveRestoreSessionRow::OnSessionNameCommitted(const FText& NewText, ETextCommit::Type CommitType)
{
	TSharedPtr<FConcertSessionTreeItem> ItemPin = Item.Pin();
	if (CommitType == ETextCommit::Type::OnEnter)
	{
		OnAccept();
	}
}

FReply SSaveRestoreSessionRow::OnAccept()
{
	// Read the session name given by the user.
	TSharedPtr<FConcertSessionTreeItem> ItemPin = Item.Pin();
	FString Name = EditableSessionName->GetText().ToString(); // Archive name or restored session name.

	// Ensure the user provided a name.
	FText InvalidNameErrorMsg = ConcertSettingsUtils::ValidateSessionName(Name);
	if (InvalidNameErrorMsg.IsEmpty()) // Name is valid?
	{
		AcceptFunc(ItemPin, Name); // Delegate archiving/restoring operation.
	}
	else
	{
		EditableSessionName->SetError(InvalidNameErrorMsg);
		FSlateApplication::Get().SetKeyboardFocus(EditableSessionName);
	}

	return FReply::Handled();
}

FReply SSaveRestoreSessionRow::OnDecline()
{
	DeclineFunc(Item.Pin()); // Remove the save/restore editable row.
	return FReply::Handled();
}

FReply SSaveRestoreSessionRow::OnKeyDownHandler(const FGeometry&, const FKeyEvent& KeyEvent)
{
	// NOTE: This handler is to capture the 'Escape' while the text field has focus.
	return KeyEvent.GetKey() == EKeys::Escape ? OnDecline() : FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
