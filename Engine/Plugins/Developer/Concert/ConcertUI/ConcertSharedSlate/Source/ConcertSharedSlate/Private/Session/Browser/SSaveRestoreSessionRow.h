// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertHeaderRowUtils.h"

#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class FConcertSessionTreeItem;
class STableViewBase;

class SWidget;
class SEditableTextBox;

/**
 * The type of row used in the session list view to create and archive or restore a session (edit the information required for the operation).
 */
class SSaveRestoreSessionRow : public SMultiColumnTableRow<TSharedPtr<FConcertSessionTreeItem>>
{
public:
	/** Should remove the editable row and save or restore the session. */
	using FAcceptFunc = TFunction<void(TSharedPtr<FConcertSessionTreeItem>, const FString&)>;
	/** Should only remove the editable row from the table. */
	using FDeclineFunc = TFunction<void(TSharedPtr<FConcertSessionTreeItem>)>;

	SLATE_BEGIN_ARGS(SSaveRestoreSessionRow)
		: _OnAcceptFunc()
		, _OnDeclineFunc()
		, _HighlightText()
	{}
		SLATE_ARGUMENT(FAcceptFunc, OnAcceptFunc)
		SLATE_ARGUMENT(FDeclineFunc, OnDeclineFunc)
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_ATTRIBUTE(FMargin, Padding)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, TSharedPtr<FConcertSessionTreeItem> Node, const TSharedRef<STableViewBase>& InOwnerTableView);
	virtual ~SSaveRestoreSessionRow() override;
	
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void OnSessionNameChanged(const FText& NewName);
	void OnSessionNameCommitted(const FText& NewText, ETextCommit::Type CommitType);
	FReply OnAccept();
	FReply OnDecline();
	FReply OnKeyDownHandler(const FGeometry&, const FKeyEvent&); // Registered as handler to the editable text (vs. OnKeyDown() virtual method for this widget).

	// Override SMultiColumnTableRow functions to ensure a wire is drawn between the item to restore and the editable row to link them together.
	virtual TBitArray<> GetWiresNeededByDepth() const override;
	virtual bool IsLastChild() const override { return true; }
	virtual int32 DoesItemHaveChildren() const override { return 0; }
	virtual bool IsItemExpanded() const override { return false; }

	/** Generates a default name for an archive or a restored session. */
	FText GetDefaultName(const FConcertSessionTreeItem& Item) const; 

private:
	
	/** Helper for temporarily showing all columns required for this column */
	UE::ConcertSharedSlate::FColumnVisibilityTransaction TemporaryColumnShower;
	
	TWeakPtr<FConcertSessionTreeItem> Item;
	TSharedPtr<SEditableTextBox> EditableSessionName;
	
	FAcceptFunc AcceptFunc;
	FDeclineFunc DeclineFunc;
	
	TAttribute<FText> HighlightText;
	bool bInitialFocusTaken = false;
};
