// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSessionGroupRow.h"

#include "ConcertFrontendStyle.h"
#include "Session/Browser/ConcertBrowserUtils.h"

#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SConcertBrowser.SSessionGroupRow"

void SSessionGroupRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	GroupType = InArgs._GroupType;
	SMultiColumnTableRow<TSharedPtr<FConcertSessionTreeItem>>::Construct(FSuperRowType::FArguments().Padding(InArgs._Padding), InOwnerTableView);
}

TSharedRef<SWidget> SSessionGroupRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == ConcertBrowserUtils::SessionColName)
	{
		const bool bIsActive = GroupType == EGroupType::Active;
		const FText Label = bIsActive
			? LOCTEXT("ActiveSessionsLabel", "Active Sessions")
			: LOCTEXT("ArchivedSessionsLabel", "Archived Sessions");
		const FSlateBrush* Glyph = bIsActive
			? FConcertFrontendStyle::Get()->GetBrush("Concert.ActiveSession.Icon")
			: FConcertFrontendStyle::Get()->GetBrush("Concert.ArchivedSession.Icon");
		return SNew( SHorizontalBox )

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(6, 0, 0, 0)
			[
				SNew( SExpanderArrow, SharedThis(this) ).IndentAmount(12)
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.f, 0.f, 4.f, 0.f)
			[
				SNew(SImage)
				.Image(Glyph)
			]
		
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(Label)
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", FConcertFrontendStyle::Get()->GetFloat("Concert.SessionBrowser.FontSize")))
			];
	}

	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE
