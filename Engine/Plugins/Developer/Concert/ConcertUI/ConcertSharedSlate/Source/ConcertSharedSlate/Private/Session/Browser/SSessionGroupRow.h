// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class STableViewBase;

class FConcertArchivedGroupTreeItem;
class FConcertSessionTreeItem;
class SEditableTextBox;
class SHeaderRow;
class SWidget;

/**
 * The type of row used in the session list view to edit a new session (the session name + server).
 */
class SSessionGroupRow : public SMultiColumnTableRow<TSharedPtr<FConcertSessionTreeItem>>
{
public:

	enum class EGroupType
	{
		Active,
		Archived
	};

	SLATE_BEGIN_ARGS(SSessionGroupRow) 
	{}
		SLATE_ARGUMENT(EGroupType, GroupType)
		SLATE_ATTRIBUTE(FMargin, Padding)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);
	
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:
	
	EGroupType GroupType{};
};
