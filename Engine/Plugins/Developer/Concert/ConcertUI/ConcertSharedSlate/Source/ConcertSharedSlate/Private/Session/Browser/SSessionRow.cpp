// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSessionRow.h"

#include "Session/Browser/ConcertBrowserUtils.h"
#include "Session/Browser/ConcertSessionBrowserSettings.h"

#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SConcertBrowser"

void SSessionRow::Construct(const FArguments& InArgs, TSharedPtr<FConcertSessionTreeItem> InItem, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	Item = MoveTemp(InItem);
	DoubleClickFunc = InArgs._OnDoubleClickFunc; // This function should join a session or add a row to restore an archive.
	RenameFunc = InArgs._OnRenameFunc; // Function invoked to send a rename request to the server.
	IsDefaultSession = InArgs._IsDefaultSession;
	HighlightText = InArgs._HighlightText;
	IsSelected = InArgs._IsSelected;

	check(DoubleClickFunc);
	check(RenameFunc);
	check(IsDefaultSession);

	// Construct base class
	SMultiColumnTableRow<TSharedPtr<FConcertSessionTreeItem>>::Construct(FSuperRowType::FArguments().Padding(InArgs._Padding), InOwnerTableView);

	// Listen and handle rename request.
	InItem->OnBeginEditSessionNameRequest.AddSP(this, &SSessionRow::OnBeginEditingSessionName);
}


FSlateColor SSessionRow::GetFontColor(bool bIsActiveSession, bool bIsDefault)
{
	if (bIsActiveSession)
	{
		return bIsDefault ? FSlateColor(FLinearColor::White) : FSlateColor(FLinearColor::White * 0.8f);
	}

	return FSlateColor::UseSubduedForeground();
}


FSlateFontInfo SSessionRow::GetFontInfo(bool bIsActiveSession, bool bIsDefault)
{
	if (bIsActiveSession)
	{
		return FCoreStyle::GetDefaultFontStyle("Regular", FConcertFrontendStyle::Get()->GetFloat("Concert.SessionBrowser.FontSize"));
	}
	return FCoreStyle::GetDefaultFontStyle("Italic", FConcertFrontendStyle::Get()->GetFloat("Concert.SessionBrowser.FontSize"));
}

TSharedRef<SWidget> SSessionRow::GenerateSessionColumn(const FSlateFontInfo& FontInfo, const FSlateColor& FontColor)
{
	return SNew(SBox)
		.VAlign(VAlign_Center)
		[
			SAssignNew(SessionNameText, SInlineEditableTextBlock)
			.Text_Lambda([this]() { return FText::AsCultureInvariant(Item.Pin()->SessionName); })
			.HighlightText(HighlightText)
			.OnTextCommitted(this, &SSessionRow::OnSessionNameCommitted)
			.IsReadOnly(false)
			.IsSelected(FIsSelected::CreateLambda([this]() { return IsSelected.Get(); }))
			.OnVerifyTextChanged(this, &SSessionRow::OnValidatingSessionName)
			.Font(FontInfo)
			.ColorAndOpacity(FontColor)
			];
}

TSharedRef<SWidget> SSessionRow::GenerateServerColumn(const FSlateFontInfo& FontInfo, const FSlateColor& FontColor)
{
	TSharedPtr<FConcertSessionTreeItem> ItemPin = Item.Pin();
	return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(FText::AsCultureInvariant(ItemPin->ServerName))
			.HighlightText(HighlightText)
			.Font(FontInfo)
			.ColorAndOpacity(FontColor)
		]
		+SHorizontalBox::Slot()
		[
			SNew(SSpacer)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		[
			ConcertBrowserUtils::MakeServerVersionIgnoredWidget(ItemPin->ServerFlags)
		];
}

TSharedRef<SWidget> SSessionRow::GenerateServerDefaultColumn(const FSlateFontInfo& FontInfo, const FSlateColor& FontColor)
{
	TSharedPtr<FConcertSessionTreeItem> ItemPin = Item.Pin();
	return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(FText::Format(INVTEXT("{0} * "), FText::AsCultureInvariant(ItemPin->ServerName)))
			.HighlightText(HighlightText)
			.Font(FontInfo)
			.ColorAndOpacity(FontColor)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("DefaultServerSession", "(Default Session/Server)"))
			.HighlightText(HighlightText)
			.Font(FCoreStyle::GetDefaultFontStyle("Regular", FConcertFrontendStyle::Get()->GetFloat("Concert.SessionBrowser.FontSize")))
			.ColorAndOpacity(FontColor)
		]
		+SHorizontalBox::Slot()
		[
			SNew(SSpacer)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		[
			ConcertBrowserUtils::MakeServerVersionIgnoredWidget(ItemPin->ServerFlags)
		];
}

TSharedRef<SWidget> SSessionRow::GenerateProjectColumn(const FSlateFontInfo &FontInfo, const FSlateColor &FontColor)
{
	return SNew(SBox)
		.VAlign(VAlign_Center)
		[
			SNew(SInlineEditableTextBlock)
			.Text_Lambda([this]() { return FText::AsCultureInvariant(Item.Pin()->ProjectName); })
			.HighlightText(HighlightText)
			.IsReadOnly(true)
			.Font(FontInfo)
			.ColorAndOpacity(FontColor)
		];
}

TSharedRef<SWidget> SSessionRow::GenerateVersionColumn(const FSlateFontInfo& FontInfo, const FSlateColor& FontColor)
{
	return SNew(SBox)
		.VAlign(VAlign_Center)
		[
			SNew(SInlineEditableTextBlock)
			.Text_Lambda([this]() { return FText::AsCultureInvariant(Item.Pin()->ProjectVersion); })
			.HighlightText(HighlightText)
			.IsReadOnly(true)
			.Font(FontInfo)
			.ColorAndOpacity(FontColor)
		];
}

TSharedRef<SWidget> SSessionRow::GenerateLastModifiedColumn(const FSlateFontInfo& FontInfo, const FSlateColor& FontColor)
{
	return SNew(SBox)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text_Lambda([this](){ return ConcertFrontendUtils::FormatTime(Item.Pin()->LastModified, GetMutableDefault<UConcertSessionBrowserSettings>()->LastModifiedTimeFormat); })
			.Font(FontInfo)
			.ColorAndOpacity(FontColor)
		];
}

TSharedRef<SWidget> SSessionRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	TSharedPtr<FConcertSessionTreeItem> ItemPin = Item.Pin();
	const bool bIsActiveSession = ItemPin->Type == FConcertSessionTreeItem::EType::ActiveSession;
	const bool bIsDefaultConfig = IsDefaultSession(ItemPin);
	FSlateFontInfo FontInfo = GetFontInfo(bIsActiveSession, bIsDefaultConfig);
	FSlateColor FontColor = GetFontColor(bIsActiveSession, bIsDefaultConfig);

	if (ColumnName == ConcertBrowserUtils::SessionColName)
	{
		const FSlateBrush* Glyph = ItemPin->Type == FConcertSessionTreeItem::EType::ActiveSession
			? FConcertFrontendStyle::Get()->GetBrush("Concert.ActiveSession.Icon")
			: FConcertFrontendStyle::Get()->GetBrush("Concert.ArchivedSession.Icon");
		return SNew( SHorizontalBox )

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(6, 0, 0, 0)
			[
				SNew( SExpanderArrow, SharedThis(this) ).IndentAmount(12)
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.f, 0.f, 4.f, 0.f)
			[
				SNew(SImage)
				.Image(Glyph)
			]

			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				GenerateSessionColumn(FontInfo, FontColor)
			];
	}

	if (ColumnName == ConcertBrowserUtils::ServerColName)
	{
		if (bIsDefaultConfig)
		{
			return GenerateServerDefaultColumn(FontInfo, FontColor);
		}
		return GenerateServerColumn(FontInfo, FontColor);
	}

	if (ColumnName == ConcertBrowserUtils::ProjectColName)
	{
		return GenerateProjectColumn(FontInfo, FontColor);
	}

	if (ColumnName == ConcertBrowserUtils::VersionColName)
	{
		return GenerateVersionColumn(FontInfo, FontColor);
	}
	
	check(ColumnName == ConcertBrowserUtils::LastModifiedColName);
	return GenerateLastModifiedColumn(FontInfo, FontColor);
}

FReply SSessionRow::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (TSharedPtr<FConcertSessionTreeItem> ItemPin = Item.Pin())
	{
		DoubleClickFunc(ItemPin);
	}
	return FReply::Handled();
}

bool SSessionRow::OnValidatingSessionName(const FText& NewSessionName, FText& OutError)
{
	OutError = ConcertSettingsUtils::ValidateSessionName(NewSessionName.ToString());
	return OutError.IsEmpty();
}

void SSessionRow::OnSessionNameCommitted(const FText& NewSessionName, ETextCommit::Type CommitType)
{
	if (TSharedPtr<FConcertSessionTreeItem> ItemPin = Item.Pin())
	{
		FString NewName = NewSessionName.ToString();
		if (NewName != ItemPin->SessionName) // Was renamed?
		{
			if (ConcertSettingsUtils::ValidateSessionName(NewName).IsEmpty()) // Name is valid?
			{
				RenameFunc(ItemPin, NewName); // Send the rename request to the server. (Server may still refuse at this point)
			}
			else
			{
				// NOTE: Error are interactively detected and raised by OnValidatingSessionName()
				FSlateApplication::Get().SetKeyboardFocus(SessionNameText);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
