// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class STableViewBase;

class FConcertSessionTreeItem;
class SWidget;
class SInlineEditableTextBlock;

class SSessionRow : public SMultiColumnTableRow<TSharedPtr<FConcertSessionTreeItem>>
{
public:
	using FDoubleClickFunc = TFunction<void(TSharedPtr<FConcertSessionTreeItem>)>;
	using FRenameFunc = TFunction<void(TSharedPtr<FConcertSessionTreeItem>, const FString&)>;
	using FIsDefaultSession = TFunction<bool(TSharedPtr<FConcertSessionTreeItem>)>;

	SLATE_BEGIN_ARGS(SSessionRow)
		: _OnDoubleClickFunc()
		, _OnRenameFunc()
		, _HighlightText()
		, _IsSelected(false)
	{}

		SLATE_ARGUMENT(FDoubleClickFunc, OnDoubleClickFunc)
		SLATE_ARGUMENT(FRenameFunc, OnRenameFunc)
		SLATE_ARGUMENT(FIsDefaultSession, IsDefaultSession)
		
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_ATTRIBUTE(FMargin, Padding)
	SLATE_ATTRIBUTE(bool, IsSelected)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FConcertSessionTreeItem> InItem, const TSharedRef<STableViewBase>& InOwnerTableView);
	
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
	virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

	void OnSessionNameCommitted(const FText& NewSessionName, ETextCommit::Type CommitType);
	
private:
	FSlateFontInfo GetFontInfo(bool bIsActiveSession, bool bIsDefault);
	FSlateColor    GetFontColor(bool bIsActiveSession, bool bIsDefault);

	TSharedRef<SWidget> GenerateSessionColumn(const FSlateFontInfo& FontInfo, const FSlateColor& FontColor);
	TSharedRef<SWidget> GenerateServerColumn(const FSlateFontInfo& FontInfo, const FSlateColor& FontColor);
	TSharedRef<SWidget> GenerateServerDefaultColumn(const FSlateFontInfo& FontInfo, const FSlateColor& FontColor);
	TSharedRef<SWidget> GenerateProjectColumn(const FSlateFontInfo& FontInfo, const FSlateColor& FontColor);
	TSharedRef<SWidget> GenerateVersionColumn(const FSlateFontInfo& FontInfo, const FSlateColor& FontColor);
	TSharedRef<SWidget> GenerateLastModifiedColumn(const FSlateFontInfo& FontInfo, const FSlateColor& FontColor);

	void OnBeginEditingSessionName() { SessionNameText->EnterEditingMode(); }
	bool OnValidatingSessionName(const FText& NewSessionName, FText& OutError);

	TWeakPtr<FConcertSessionTreeItem> Item;

	/** Invoked when the user double click on the row */
	FDoubleClickFunc DoubleClickFunc;
	/** Invoked when the user commit the session rename. (This will send the request to server) */
	FRenameFunc RenameFunc;
	/** Given a session checks whether it is the default session the client is supposed to join */
	FIsDefaultSession IsDefaultSession;
	
	TAttribute<FText> HighlightText;
	TAttribute<bool> IsSelected;
	TSharedPtr<SInlineEditableTextBlock> SessionNameText;
};
