// Copyright Epic Games, Inc. All Rights Reserved.

#include "Session/History/AbstractSessionHistoryController.h"

#include "Async/Future.h"
#include "Session/History/SSessionHistory.h"
#include "ConcertSyncSessionTypes.h"

FAbstractSessionHistoryController::FAbstractSessionHistoryController(SSessionHistory::FArguments Arguments)
	: SessionHistory(MakeSessionHistory(MoveTemp(Arguments)))
{}

void FAbstractSessionHistoryController::ReloadActivities()
{
	constexpr int64 MaximumNumberOfActivities = SSessionHistory::MaximumNumberOfActivities;
	TMap<FGuid, FConcertClientInfo> EndpointClientInfoMap;
	TArray<FConcertSessionActivity> FetchedActivities;
	GetActivities(MaximumNumberOfActivities, EndpointClientInfoMap, FetchedActivities);

	// MoveTemp strictly not needed - but it will be faster on Debug builds
	GetSessionHistory()->ReloadActivities(
		MoveTemp(EndpointClientInfoMap),
		MoveTemp(FetchedActivities)
		);
}

TSharedRef<SSessionHistory> FAbstractSessionHistoryController::MakeSessionHistory(SSessionHistory::FArguments Arguments) const
{
	return SArgumentNew(
		Arguments
			.GetPackageEvent_Raw(this, &FAbstractSessionHistoryController::GetPackageEvent)
			.GetTransactionEvent_Raw(this, &FAbstractSessionHistoryController::GetTransactionEvent),
		SSessionHistory);
}
