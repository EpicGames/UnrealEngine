// Copyright Epic Games, Inc. All Rights Reserved.

#include "Session/History/SEditableSessionHistory.h"

#include "ConcertSyncSessionTypes.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SNegativeActionButton.h"

#define LOCTEXT_NAMESPACE "SEditableSessionHistory"

void SEditableSessionHistory::Construct(const FArguments& InArgs)
{
	check(InArgs._MakeSessionHistory.IsBound()
		&& InArgs._CanDeleteActivities.IsBound()
		&& InArgs._CanMuteActivities.IsBound()
		&& InArgs._CanUnmuteActivities.IsBound()
		);
	
	CanDeleteActivitiesFunc = InArgs._CanDeleteActivities;
	DeleteActivitiesFunc = InArgs._DeleteActivities;
	CanMuteActivitiesFunc = InArgs._CanMuteActivities;
	MuteActivitiesFunc = InArgs._MuteActivities;
	CanUnmuteActivitiesFunc = InArgs._CanUnmuteActivities;
	UnmuteActivitiesFunc = InArgs._UnmuteActivities;
	
	SessionHistory = InArgs._MakeSessionHistory.Execute(
		SSessionHistory::FArguments()
		.SelectionMode(ESelectionMode::Multi)
		.OnContextMenuOpening(this, &SEditableSessionHistory::OnContextMenuOpening)
		.SearchButtonArea()
		[
			SNew(SHorizontalBox)

			// Mute
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SNegativeActionButton)
				.ActionButtonStyle(EActionButtonStyle::Warning) // Yellow
				.OnClicked(this, &SEditableSessionHistory::OnClickMuteActivitesButton)
				.ToolTipText(this, &SEditableSessionHistory::GetMuteActivitiesToolTip)
				.IsEnabled(this, &SEditableSessionHistory::IsMuteButtonEnabled)
				.Icon(FConcertFrontendStyle::Get()->GetBrush("Concert.MuteActivities"))
			]

			// Unmute
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SNegativeActionButton)
				.ActionButtonStyle(EActionButtonStyle::Warning) // Yellow
				.OnClicked(this, &SEditableSessionHistory::OnClickUnmuteActivitesButton)
				.ToolTipText(this, &SEditableSessionHistory::GetUnmuteActivitiesToolTip)
				.IsEnabled(this, &SEditableSessionHistory::IsUnmuteButtonEnabled)
				.Icon(FConcertFrontendStyle::Get()->GetBrush("Concert.UnmuteActivities"))
			]
		]
		);
	
	ChildSlot
	[
		SessionHistory.ToSharedRef()
	];
}

FReply SEditableSessionHistory::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Delete)
	{
		const TSet<TSharedRef<FConcertSessionActivity>> SelectedActivities = SessionHistory->GetSelectedActivities();
		if (CanDeleteActivitiesFunc.Execute(SelectedActivities).CanPerformAction())
		{
			DeleteActivitiesFunc.Execute(SelectedActivities);
		}
		return FReply::Handled();
	}
	
	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

TSharedPtr<SWidget> SEditableSessionHistory::OnContextMenuOpening()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("MuteMenuLabel", "Mute"),
		FText::GetEmpty(),
		FSlateIcon(FConcertFrontendStyle::GetStyleSetName(), "Concert.MuteActivities"),
		FUIAction(
		FExecuteAction::CreateLambda([this](){ OnClickMuteActivitesButton(); }),
			FCanExecuteAction::CreateLambda([this] { return IsMuteButtonEnabled(); })
		),
		NAME_None,
		EUserInterfaceActionType::Button
		);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("UnmuteMenuLabel", "Unmute"),
		FText::GetEmpty(),
		FSlateIcon(FConcertFrontendStyle::GetStyleSetName(), "Concert.UnmuteActivities"),
		FUIAction(
		FExecuteAction::CreateLambda([this](){ OnClickUnmuteActivitesButton(); }),
			FCanExecuteAction::CreateLambda([this] { return IsUnmuteButtonEnabled(); })
		),
		NAME_None,
		EUserInterfaceActionType::Button
		);
	MenuBuilder.AddSubMenu(
		LOCTEXT("EditMenuLabel", "Edit"),
		FText::GetEmpty(),
		FNewMenuDelegate::CreateLambda([this](FMenuBuilder& SubMenuBuilder)
		{
			SubMenuBuilder.AddMenuEntry(
				LOCTEXT("DeleteMenuLabel", "Delete"),
				FText::GetEmpty(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"),
				FUIAction(
				FExecuteAction::CreateLambda([this](){ OnClickDeleteActivitiesButton(); }),
					FCanExecuteAction::CreateLambda([this] { return IsDeleteButtonEnabled(); })
				),
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}),
		false,
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit")
	);
	
	return MenuBuilder.MakeWidget();
}

FReply SEditableSessionHistory::OnClickDeleteActivitiesButton() const
{
	if (DeleteActivitiesFunc.IsBound() && CanDeleteActivitiesFunc.Execute(SessionHistory->GetSelectedActivities()).CanPerformAction())
	{
		DeleteActivitiesFunc.Execute(SessionHistory->GetSelectedActivities());
	}
	return FReply::Handled();
}

FText SEditableSessionHistory::GetDeleteActivitiesToolTip() const
{
	return GenerateTooltip(
		CanDeleteActivitiesFunc,
		LOCTEXT("Delete.SelectActivityToolTip", "Select some activities to delete from below (multi-select using CTRL + Click)."),
		LOCTEXT("Delete.SelectedActivitiesToolTip", "Delete selection from history (IDs: {0})"),
		LOCTEXT("Delete.CannotSelectedActivitiesToolTip", "Cannot delete:\n{0}")
		);
}

bool SEditableSessionHistory::IsDeleteButtonEnabled() const
{
	const TSet<TSharedRef<FConcertSessionActivity>> SelectedActivities = SessionHistory->GetSelectedActivities();
	return SelectedActivities.Num() > 0 && CanDeleteActivitiesFunc.Execute(SelectedActivities).CanPerformAction();
}

FReply SEditableSessionHistory::OnClickMuteActivitesButton() const
{
	if (MuteActivitiesFunc.IsBound() && CanMuteActivitiesFunc.Execute(SessionHistory->GetSelectedActivities()).CanPerformAction())
	{
		MuteActivitiesFunc.Execute(SessionHistory->GetSelectedActivities());
	}
	return FReply::Handled();
}

FText SEditableSessionHistory::GetMuteActivitiesToolTip() const
{
	return GenerateTooltip(
		CanMuteActivitiesFunc,
		LOCTEXT("Mute.SelectActivityToolTip", "Select some activities to mute (multi-select using CTRL + Click)."),
		LOCTEXT("Mute.SelectedActivitiesToolTip", "Mute selection ({0})"),
		LOCTEXT("Mute.CannotSelectedActivitiesToolTip", "Cannot mute:\n{0}")
		);
}

bool SEditableSessionHistory::IsMuteButtonEnabled() const
{
	const TSet<TSharedRef<FConcertSessionActivity>> SelectedActivities = SessionHistory->GetSelectedActivities();
	return SelectedActivities.Num() > 0 && CanMuteActivitiesFunc.Execute(SelectedActivities).CanPerformAction();
}

FReply SEditableSessionHistory::OnClickUnmuteActivitesButton() const
{
	if (UnmuteActivitiesFunc.IsBound() && CanUnmuteActivitiesFunc.Execute(SessionHistory->GetSelectedActivities()).CanPerformAction())
	{
		UnmuteActivitiesFunc.Execute(SessionHistory->GetSelectedActivities());
	}
	return FReply::Handled();
}

FText SEditableSessionHistory::GetUnmuteActivitiesToolTip() const
{
	return GenerateTooltip(
		CanUnmuteActivitiesFunc,
		LOCTEXT("Unmute.SelectActivityToolTip", "Select some activities to unmute (multi-select using CTRL + Click)."),
		LOCTEXT("Unmute.SelectedActivitiesToolTip", "Unmute selection ({0})"),
		LOCTEXT("Unmute.CannotSelectedActivitiesToolTip", "Cannot unmute:\n{0}")
		);
}

bool SEditableSessionHistory::IsUnmuteButtonEnabled() const
{
	const TSet<TSharedRef<FConcertSessionActivity>> SelectedActivities = SessionHistory->GetSelectedActivities();
	return SelectedActivities.Num() > 0 && CanUnmuteActivitiesFunc.Execute(SelectedActivities).CanPerformAction();
}

FText SEditableSessionHistory::GenerateTooltip(const FCanPerformActionOnActivities& CanPerformAction, FText SelectActivityToolTip, FText PerformActionToolTipFmt, FText CannotPerformActionToolTipFmt) const
{
	TSet<TSharedRef<FConcertSessionActivity>> SelectedActivities = SessionHistory->GetSelectedActivities();
	if (SelectedActivities.Num() == 0)
	{
		return SelectActivityToolTip;
	}
	
	const FCanPerformActionResult CanDeleteActivities = CanPerformAction.Execute(SelectedActivities);
	if (CanDeleteActivities.CanPerformAction())
	{
		SelectedActivities.Sort([](const TSharedRef<FConcertSessionActivity>& First, const TSharedRef<FConcertSessionActivity>& Second) { return First->Activity.ActivityId <= Second->Activity.ActivityId; });

		TArray<FString> Items;
		Algo::Transform(SelectedActivities, Items, [](const TSharedRef<FConcertSessionActivity>& Activity)
		{
			return FString::FromInt(Activity->Activity.ActivityId);
		});
		const FString AllItems = FString::Join(Items, TEXT(", "));
		
		return FText::Format(PerformActionToolTipFmt, FText::FromString(AllItems));
	}
	return FText::Format(CannotPerformActionToolTipFmt, CanDeleteActivities.DeletionReason.GetValue());
}

#undef LOCTEXT_NAMESPACE
