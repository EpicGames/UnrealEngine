// Copyright Epic Games, Inc. All Rights Reserved.

#include "Session/History/SSessionHistory.h"




#include "ConcertSyncSessionTypes.h"
#include "Styling/ToolBarStyle.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SSeparator.h"

#define LOCTEXT_NAMESPACE "SSessionHistory"

namespace ConcertSessionHistoryUI
{
	bool PackageNamePassesFilter(const FName& PackageNameFilter, const TStructOnScope<FConcertSyncActivitySummary>& InActivitySummary)
	{
		if (PackageNameFilter.IsNone())
		{
			return true;
		}

		if (const FConcertSyncLockActivitySummary* Summary = InActivitySummary.Cast<FConcertSyncLockActivitySummary>())
		{
			return Summary->PrimaryPackageName == PackageNameFilter;
		}

		if (const FConcertSyncTransactionActivitySummary* Summary = InActivitySummary.Cast<FConcertSyncTransactionActivitySummary>())
		{
			return Summary->PrimaryPackageName == PackageNameFilter;
		}

		if (const FConcertSyncPackageActivitySummary* Summary = InActivitySummary.Cast<FConcertSyncPackageActivitySummary>())
		{
			return Summary->PackageName == PackageNameFilter;
		}

		return false;
	}
}

void SSessionHistory::Construct(const FArguments& InArgs)
{
	using namespace UE::ConcertSharedSlate;
	
	AllowActivityFunc = FAllowActivity::CreateLambda([this, PackageNameFilter = InArgs._PackageFilter, FilterFunc = InArgs._AllowActivity](const FConcertSyncActivity& Activity, const TStructOnScope<FConcertSyncActivitySummary>& Summary)
	{
		return ConcertSessionHistoryUI::PackageNamePassesFilter(PackageNameFilter, Summary)
			&& (!FilterFunc.IsBound() || FilterFunc.Execute(Activity, Summary));
	});

	ActivityMap.Reserve(MaximumNumberOfActivities);
	ActivityListViewOptions = InArgs._ViewOptions.Get() ? InArgs._ViewOptions.Get() : MakeShared<FConcertSessionActivitiesOptions>();

	SAssignNew(ActivityListView, SConcertSessionActivities)
		.OnGetPackageEvent(InArgs._GetPackageEvent)
		.OnGetTransactionEvent(InArgs._GetTransactionEvent)
		.OnContextMenuOpening(InArgs._OnContextMenuOpening)
		.OnMakeColumnOverlayWidget(InArgs._OnMakeColumnOverlayWidget)
		.OnMapActivityToClient(this, &SSessionHistory::GetClientInfo)
		.HighlightText(this, &SSessionHistory::HighlightSearchedText)
		.TimeFormat(ActivityListViewOptions.Get(), &FConcertSessionActivitiesOptions::GetTimeFormat)
		.Columns(InArgs._Columns)
		.ConnectionActivitiesVisibility(ActivityListViewOptions.Get(), &FConcertSessionActivitiesOptions::GetConnectionActivitiesVisibility)
		.LockActivitiesVisibility(ActivityListViewOptions.Get(), &FConcertSessionActivitiesOptions::GetLockActivitiesVisibility)
		.PackageActivitiesVisibility(ActivityListViewOptions.Get(), &FConcertSessionActivitiesOptions::GetPackageActivitiesVisibility)
		.TransactionActivitiesVisibility(ActivityListViewOptions.Get(), &FConcertSessionActivitiesOptions::GetTransactionActivitiesVisibility)
		.ReplicationActivitiesVisibility(ActivityListViewOptions.Get(), &FConcertSessionActivitiesOptions::GetReplicationActivitiesVisibility)
		.DetailsAreaVisibility(InArgs._DetailsAreaVisibility)
		.IsAutoScrollEnabled(true)
		.ColumnVisibilitySnapshot(InArgs._ColumnVisibilitySnapshot)
		.SaveColumnVisibilitySnapshot(InArgs._SaveColumnVisibilitySnapshot)
		.SelectionMode(InArgs._SelectionMode)
		.UndoHistoryReflectionProvider(InArgs._UndoHistoryReflectionProvider)
		.DarkenMutedActivities(InArgs._DarkenMutedActivities);

	
	TSharedPtr<SVerticalBox> ButtonExtension;
	ChildSlot
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(ButtonExtension, SVerticalBox)
		]
		
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2, 0, 2, 4)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SAssignNew(SearchBox, SSearchBox)
				.HintText(LOCTEXT("SearchHint", "Search..."))
				.OnTextChanged(this, &SSessionHistory::OnSearchTextChanged)
				.OnTextCommitted(this, &SSessionHistory::OnSearchTextCommitted)
				.DelayChangeNotificationsWhileTyping(true)
			]
			
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				ActivityListViewOptions->MakeViewOptionsComboButton()
			]
		]

		+SVerticalBox::Slot()
		[
			ActivityListView.ToSharedRef()
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4, 3)
		[
			SNew(SSeparator)
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4, 0, 4, 3)
		[
			ActivityListViewOptions->MakeStatusBar(
				TAttribute<int32>(ActivityListView.Get(), &SConcertSessionActivities::GetTotalActivityNum),
				TAttribute<int32>(ActivityListView.Get(), &SConcertSessionActivities::GetDisplayedActivityNum)
				)
		]
	];

	const bool bHasExtensions = InArgs._SearchButtonArea.Widget != SNullWidget::NullWidget;
	if (bHasExtensions)
	{
		ButtonExtension->AddSlot()
			.AutoHeight()
			[
				InArgs._SearchButtonArea.Widget
			];
		
		ButtonExtension->AddSlot()
			.AutoHeight()
			.Padding(0, 3)
			[
				SNew(SSeparator)
				.Thickness(FCoreStyle::Get().GetWidgetStyle<FToolBarStyle>("ToolBar").SeparatorThickness)
				.SeparatorImage(&FCoreStyle::Get().GetWidgetStyle<FToolBarStyle>("ToolBar").SeparatorBrush)
			];
	}
}

void SSessionHistory::ReloadActivities(TMap<FGuid, FConcertClientInfo> InEndpointClientInfoMap, TArray<FConcertSessionActivity> InFetchedActivities)
{
	EndpointClientInfoMap = MoveTemp(InEndpointClientInfoMap);
	ActivityMap.Reset();
	ActivityListView->ResetActivityList(); 

	for (FConcertSessionActivity& FetchedActivity : InFetchedActivities)
	{
		if (AllowActivityFunc.Execute(FetchedActivity.Activity, FetchedActivity.ActivitySummary))
		{
			TSharedRef<FConcertSessionActivity> NewActivity = MakeShared<FConcertSessionActivity>(MoveTemp(FetchedActivity));
			ActivityMap.Add(NewActivity->Activity.ActivityId, NewActivity);
			ActivityListView->Append(NewActivity);
		}
	}
}

void SSessionHistory::HandleActivityAddedOrUpdated(const FConcertClientInfo& InClientInfo, const FConcertSyncActivity& InActivity, const FStructOnScope& InActivitySummary)
{
	TStructOnScope<FConcertSyncActivitySummary> ActivitySummary;
	ActivitySummary.InitializeFromChecked(InActivitySummary);

	if (AllowActivityFunc.Execute(InActivity, ActivitySummary))
	{
		EndpointClientInfoMap.Add(InActivity.EndpointId, InClientInfo);

		if (TSharedPtr<FConcertSessionActivity> ExistingActivity = ActivityMap.FindRef(InActivity.ActivityId))
		{
			ExistingActivity->Activity = InActivity;
			ExistingActivity->ActivitySummary = MoveTemp(ActivitySummary);
			ActivityListView->RequestRefresh();
		}
		else
		{
			TSharedRef<FConcertSessionActivity> NewActivity = MakeShared<FConcertSessionActivity>();
			NewActivity->Activity = InActivity;
			NewActivity->ActivitySummary = MoveTemp(ActivitySummary);

			ActivityMap.Add(NewActivity->Activity.ActivityId, NewActivity);
			ActivityListView->Append(NewActivity);
		}
	}
}

void SSessionHistory::OnColumnVisibilitySettingsChanged(const FColumnVisibilitySnapshot& ColumnSnapshot)
{
	ActivityListView->OnColumnVisibilitySettingsChanged(ColumnSnapshot);
}

TSet<TSharedRef<FConcertSessionActivity>> SSessionHistory::GetSelectedActivities() const
{
	TSet<TSharedRef<FConcertSessionActivity>> Activities;
	Algo::Transform(ActivityListView->GetSelectedActivities(), Activities, [&](TSharedPtr<FConcertSessionActivity> Activity)
	{
		return Activity.ToSharedRef();
	});
	return Activities;
}

void SSessionHistory::OnSearchTextChanged(const FText& InSearchText)
{
	SearchedText = InSearchText;
	SearchBox->SetError(ActivityListView->UpdateTextFilter(InSearchText));
}

void SSessionHistory::OnSearchTextCommitted(const FText& InSearchText, ETextCommit::Type CommitType)
{
	if (!InSearchText.EqualTo(SearchedText))
	{
		OnSearchTextChanged(InSearchText);
	}
}

FText SSessionHistory::HighlightSearchedText() const
{
	return SearchedText;
}

TOptional<FConcertClientInfo> SSessionHistory::GetClientInfo(FGuid Guid) const
{
	if (const FConcertClientInfo* ClientInfo = EndpointClientInfoMap.Find(Guid))
	{
		return *ClientInfo;
	}
	return {};
}

#undef LOCTEXT_NAMESPACE /* SSessionHistory */
