// Copyright Epic Games, Inc. All Rights Reserved.

#include "Session/History/SSessionHistoryWrapper.h"
#include "Session/History/AbstractSessionHistoryController.h"
#include "Session/History/SSessionHistory.h"

void SSessionHistoryWrapper::Construct(const FArguments& InArgs, TSharedRef<FAbstractSessionHistoryController> InController)
{
	Controller = InController;
	ChildSlot
	[
		Controller->GetSessionHistory()
	];
}
