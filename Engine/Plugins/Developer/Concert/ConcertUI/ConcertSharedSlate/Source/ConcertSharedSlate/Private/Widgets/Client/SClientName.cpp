// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Client/SClientName.h"

#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SClientName"

namespace UE::ConcertSharedSlate::ParenthesesClientNameContent
{
	const FText LocalClient = LOCTEXT("ParenthesesClientNameContent.LocalClient", "You");
	const FText OfflineClient = LOCTEXT("ParenthesesClientNameContent.OfflineClient", "Offline");
}

namespace UE::ConcertSharedSlate
{
	void SClientName::Construct(const FArguments& InArgs)
	{
		ClientInfoAttribute = InArgs._ClientInfo;
		ParenthesisContentAttribute = InArgs._ParenthesisContent;
		check(ClientInfoAttribute.IsSet() || ClientInfoAttribute.IsBound());
		
		ChildSlot
		[
			SNew(SHorizontalBox)
			
			// The user "Avatar color" displayed as a small square colored by the user avatar color.
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SImage)
				.ColorAndOpacity(this, &SClientName::GetAvatarColor)
				.Image(FAppStyle::GetBrush("Icons.FilledCircle"))
			]
					
			// The user "Display Name".
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(1, 0, 0, 0)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("NoBorder"))
				.ColorAndOpacity(FLinearColor(0.75f, 0.75f, 0.75f))
				[
					SNew(STextBlock)
					.Font(InArgs._Font)
					.Text(TAttribute<FText>::CreateSP(this, &SClientName::GetClientDisplayName))
					.HighlightText(InArgs._HighlightText)
				]
			]
		];
	}

	FText SClientName::GetDisplayText(const FConcertClientInfo& Info, bool bDisplayAsLocalClient)
	{
		return bDisplayAsLocalClient
			? GetDisplayText(Info, ParenthesesClientNameContent::LocalClient)
			: GetDisplayText(Info);
	}

	FText SClientName::GetDisplayText(const FConcertClientInfo& Info, const FText& ParenthesesContent)
	{
		if (!ParenthesesContent.IsEmpty())
		{
			return FText::Format(
				LOCTEXT("ClientDisplayNameFmt", "{0} ({1})"),
				FText::FromString(Info.DisplayName),
				ParenthesesContent
				);
		}
		
		return FText::FromString(Info.DisplayName);
	}

	FText SClientName::GetClientDisplayName() const
	{
		const TOptional<FConcertClientInfo> ClientInfo = ClientInfoAttribute.Get();
		const FText ParenthesesContent = ParenthesisContentAttribute.IsSet() || ParenthesisContentAttribute.IsBound()
			? ParenthesisContentAttribute.Get()
			: FText::GetEmpty();
		return ClientInfo
			? GetDisplayText(*ClientInfo, ParenthesesContent)
			: LOCTEXT("Unavailable", "Unavailable");
	}

	FSlateColor SClientName::GetAvatarColor() const
	{
		const TOptional<FConcertClientInfo> ClientInfo = ClientInfoAttribute.Get();
		return ClientInfo ? ClientInfo->AvatarColor : FSlateColor(FLinearColor::Gray);
	}
}

#undef LOCTEXT_NAMESPACE