// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Client/SHorizontalClientList.h"

#include "Widgets/Client/SClientName.h"

#include "Widgets/Client/SLocalClientName.h"
#include "Widgets/Client/SRemoteClientName.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SHorizontalClientList"

namespace UE::ConcertSharedSlate::HorizontalClientList
{
	static TArray<FConcertSessionClientInfo> GetSortedClients(
			const TConstArrayView<FGuid>& Clients,
			const ConcertSharedSlate::FGetOptionalClientInfo& GetClientInfoDelegate,
			const FClientSortPredicate& SortPredicate
			)
	{
		// Prefetch the client info to avoid many GetClientInfoDelegate calls during Sort()
		TArray<FConcertSessionClientInfo> ClientsToDisplay;
		for (const FGuid& Client : Clients)
		{
			const TOptional<FConcertClientInfo> ClientInfo = GetClientInfoDelegate.Execute(Client);
			FConcertSessionClientInfo Info;
			Info.ClientEndpointId = Client;
			Info.ClientInfo = ClientInfo ? *ClientInfo : FConcertClientInfo{ .DisplayName = TEXT("Unavailable") };
			ClientsToDisplay.Add(Info);
		}

		ClientsToDisplay.Sort([&SortPredicate](const FConcertSessionClientInfo& Left, const FConcertSessionClientInfo& Right)
		{
			return SortPredicate.Execute(Left, Right);
		});
		return ClientsToDisplay;
	}
}

namespace UE::ConcertSharedSlate
{
	TOptional<FString> SHorizontalClientList::GetDisplayString(
		const TConstArrayView<FGuid>& Clients,
		const FGetOptionalClientInfo& GetClientInfoDelegate,
		const FClientSortPredicate& SortPredicate,
		const FGetClientParenthesesContent& GetClientParenthesesContent
		)
	{
		const TArray<FConcertSessionClientInfo> ClientsToDisplay = HorizontalClientList::GetSortedClients(Clients, GetClientInfoDelegate, SortPredicate);
		if (!ClientsToDisplay.IsEmpty())
		{
			return {};
		}
		
		return FString::JoinBy(ClientsToDisplay, TEXT(", "), [&GetClientParenthesesContent](const FConcertSessionClientInfo& ClientInfo)
			{
				// GetSortedClients should return empty if GetSortedClients is invalid
				const FText ParenthesesContent = EvaluateGetClientParenthesesContent(GetClientParenthesesContent, ClientInfo.ClientEndpointId);
				return SClientName::GetDisplayText(ClientInfo.ClientInfo, ParenthesesContent).ToString();
			});
	}

	void SHorizontalClientList::Construct(const FArguments& InArgs)
	{
		GetClientParenthesesContentDelegate = InArgs._GetClientParenthesesContent;
		GetClientInfoDelegate = InArgs._GetClientInfo;
		SortPredicateDelegate = InArgs._SortPredicate.IsBound()
			? InArgs._SortPredicate
			: FClientSortPredicate::CreateLambda([this](const FConcertSessionClientInfo& Left, const FConcertSessionClientInfo& Right)
			{
				return SortSpecifiedParenthesesFirstThenThenAlphabetical(
					Left, Right, GetClientParenthesesContentDelegate, ParenthesesClientNameContent::LocalClient
					);
			});
		
		ShouldDisplayAvatarColorAttribute = InArgs._DisplayAvatarColor;
		HighlightTextAttribute = InArgs._HighlightText;
		
		NameFont = InArgs._Font;

		check(GetClientInfoDelegate.IsBound());
		
		ChildSlot
		[
			SAssignNew(WidgetSwitcher, SWidgetSwitcher)
			.WidgetIndex(0)
			+SWidgetSwitcher::Slot()
			[
				InArgs._EmptyListSlot.Widget
			]
			+SWidgetSwitcher::Slot()
			[
				SAssignNew(ScrollBox, SScrollBox)
				.Orientation(Orient_Horizontal)
				.ToolTipText(InArgs._ListToolTipText)
			]
		];
	}

	void SHorizontalClientList::RefreshList(const TConstArrayView<FGuid>& Clients) const
	{
		ScrollBox->ClearChildren();

		if (Clients.IsEmpty())
		{
			WidgetSwitcher->SetActiveWidgetIndex(0);
			return;
		}
		WidgetSwitcher->SetActiveWidgetIndex(1);

		const TArray<FConcertSessionClientInfo> ClientsToDisplay = HorizontalClientList::GetSortedClients(Clients, GetClientInfoDelegate, SortPredicateDelegate);
		bool bIsFirst = true;
		for (const FConcertSessionClientInfo& Info : ClientsToDisplay)
		{
			if (!bIsFirst)
			{
				ScrollBox->AddSlot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.Padding(-1, 1, 0, 0)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Comma", ", "))
					.Font(NameFont)
				];
			}

			const FGuid& EndpointId = Info.ClientEndpointId;
			const auto GetParenthesesContent = [this, EndpointId]()
			{
				return EvaluateGetClientParenthesesContent(GetClientParenthesesContentDelegate, EndpointId);
			};
			ScrollBox->AddSlot()
			[
				SNew(SClientName)
				.ClientInfo_Lambda([this, EndpointId](){ return GetClientInfoDelegate.Execute(EndpointId); })
				.ParenthesisContent_Lambda(GetParenthesesContent)
				.DisplayAvatarColor(ShouldDisplayAvatarColorAttribute)
				.HighlightText(HighlightTextAttribute)
				.Font(NameFont)
			];
			bIsFirst = false;
		}
	}
}

#undef LOCTEXT_NAMESPACE