// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Client/SLocalClientName.h"

#include "Widgets/Client/SClientName.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SLocalClientName"

namespace UE::ConcertSharedSlate
{
	void SLocalClientName::Construct(const FArguments& InArgs)
	{
		ChildSlot
		[
			SNew(SClientName)
			.ClientInfo(InArgs._DisplayInfo)
			.ParenthesisContent(ParenthesesClientNameContent::LocalClient)
			.DisplayAvatarColor(InArgs._DisplayAvatarColor)
			.HighlightText(InArgs._HighlightText)
			.Font(InArgs._Font)
		];
	}
}

#undef LOCTEXT_NAMESPACE