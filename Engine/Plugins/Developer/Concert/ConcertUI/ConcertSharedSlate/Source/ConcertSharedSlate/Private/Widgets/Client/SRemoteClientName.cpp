// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Client/SRemoteClientName.h"

#include "Widgets/Client/SClientName.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SLocalClientName"

namespace UE::ConcertSharedSlate
{
	void SRemoteClientName::Construct(const FArguments& InArgs)
	{
		ClientDisplayInfo = InArgs._DisplayInfo;
		
		ChildSlot
		[
			SNew(SClientName)
			.ClientInfo(this, &SRemoteClientName::GetClientInfo)
			.DisplayAvatarColor(InArgs._DisplayAvatarColor)
			.HighlightText(InArgs._HighlightText)
			.Font(InArgs._Font)
		];
	}

	TOptional<FConcertClientInfo> SRemoteClientName::GetClientInfo() const
	{
		const TOptional<FConcertClientInfo> NewInfo = ClientDisplayInfo.IsBound()
			? ClientDisplayInfo.Get()
			: TOptional<FConcertClientInfo>{};
					
		if (NewInfo)
		{
			LastKnownClientInfo = NewInfo;
		}
		return LastKnownClientInfo;
	}
}

#undef LOCTEXT_NAMESPACE