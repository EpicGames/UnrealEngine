// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyle.h"

#define UE_API CONCERTSHAREDSLATE_API

class FConcertFrontendStyle
{
public:
	
	static UE_API void Initialize();
	static UE_API void Shutdown();

	static UE_API TSharedPtr< class ISlateStyle > Get();

	static UE_API FName GetStyleSetName();
	
private:
	
	static UE_API FString InContent(const FString& RelativePath, const ANSICHAR* Extension);
	static UE_API TSharedPtr< class FSlateStyleSet > StyleSet;
};

#undef UE_API
