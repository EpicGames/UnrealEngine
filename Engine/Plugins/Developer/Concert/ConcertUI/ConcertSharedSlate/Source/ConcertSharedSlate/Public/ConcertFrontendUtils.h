// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertFrontendStyle.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "ConcertFrontendUtils.generated.h"

#define LOCTEXT_NAMESPACE "ConcertFrontendUtils"

/** Defines how the time should be displayed in the date/time column. */
UENUM()
enum class ETimeFormat
{
	Relative, // Display relative time (23 seconds ago)
	Absolute  // Display absolute time (April 7, 2019 - 10:33:52)
};

namespace ConcertFrontendUtils
{
	template <typename ItemType, typename PredFactoryType>
	inline void SyncArraysByPredicate(TArray<TSharedPtr<ItemType>>& InOutArray, TArray<TSharedPtr<ItemType>>&& InNewArray, const PredFactoryType& InPredFactory)
	{
		if (InOutArray.Num() == 0)
		{
			// Empty array - can just move
			InOutArray = MoveTempIfPossible(InNewArray);
		}
		else
		{
			// Add or update the existing entries
			for (TSharedPtr<ItemType>& NewItem : InNewArray)
			{
				TSharedPtr<ItemType>* ExistingItemPtr = InOutArray.FindByPredicate(InPredFactory(NewItem));
				if (ExistingItemPtr)
				{
					**ExistingItemPtr = *NewItem;
				}
				else
				{
					InOutArray.Add(NewItem);
				}
			}
			// Remove entries that are no longer needed
			for (auto ExistingItemIt = InOutArray.CreateIterator(); ExistingItemIt; ++ExistingItemIt)
			{
				TSharedPtr<ItemType>* NewItemPtr = InNewArray.FindByPredicate(InPredFactory(*ExistingItemIt));
				if (!NewItemPtr)
				{
					ExistingItemIt.RemoveCurrent();
					continue;
				}
			}
		}
	}

	template <typename ItemType>
	inline TArray<TSharedPtr<ItemType>> DeepCopyArray(const TArray<TSharedPtr<ItemType>>& InArray)
	{
		TArray<TSharedPtr<ItemType>> ArrayCopy;
		{
			ArrayCopy.Reserve(InArray.Num());
			for (const TSharedPtr<ItemType>& Item : InArray)
			{
				ArrayCopy.Add(MakeShared<ItemType>(*Item));
			}
		}
		return ArrayCopy;
	}

	template <typename ItemType>
	inline TArray<TSharedPtr<ItemType>> DeepCopyArrayAndClearSource(TArray<TSharedPtr<ItemType>>& InOutArray)
	{
		TArray<TSharedPtr<ItemType>> ArrayCopy = DeepCopyArray(InOutArray);
		InOutArray.Reset();
		return ArrayCopy;
	}

	/** Returns the image used to render the expandable area title bar with respect to its hover/expand state. */
	inline const FSlateBrush* GetExpandableAreaBorderImage(const SExpandableArea& Area)
	{
		if (Area.IsTitleHovered())
		{
			return Area.IsExpanded() ? FConcertFrontendStyle::Get()->GetBrush("DetailsView.CategoryTop_Hovered") : FConcertFrontendStyle::Get()->GetBrush("DetailsView.CollapsedCategory_Hovered");
		}
		return Area.IsExpanded() ? FConcertFrontendStyle::Get()->GetBrush("DetailsView.CategoryTop") : FConcertFrontendStyle::Get()->GetBrush("DetailsView.CollapsedCategory");
	}

	static FText FormatRelativeTime(const FDateTime& EventTime, const FDateTime* CurrTime = nullptr)
	{
		FTimespan TimeSpan = (CurrTime ? *CurrTime : FDateTime::UtcNow()) - EventTime;
		int32 Days = TimeSpan.GetDays();
		int32 Hours = TimeSpan.GetHours();

		if (Days >= 1)
		{
			return Hours > 0 ?
				FText::Format(LOCTEXT("DaysHours", "{0} {0}|plural(one=Day,other=Days), {1} {1}|plural(one=Hour,other=Hours) Ago"), Days, Hours) :
				FText::Format(LOCTEXT("Days", "{0} {0}|plural(one=Day,other=Days) Ago"), Days);
		}

		int32 Minutes = TimeSpan.GetMinutes();
		if (Hours >= 1)
		{
			return Minutes > 0 ?
				FText::Format(LOCTEXT("HoursMins", "{0} {0}|plural(one=Hour,other=Hours), {1} {1}|plural(one=Minute,other=Minutes) Ago"), Hours, Minutes) :
				FText::Format(LOCTEXT("Hours", "{0} {0}|plural(one=Hour,other=Hours) Ago"), Hours);
		}

		int32 Seconds = TimeSpan.GetSeconds();
		if (Minutes >= 1)
		{
			return Seconds > 0 ?
				FText::Format(LOCTEXT("MinsSecs", "{0} {0}|plural(one=Minute,other=Minutes), {1} {1}|plural(one=Second,other=Seconds) Ago"), Minutes, Seconds) :
				FText::Format(LOCTEXT("Mins", "{0} {0}|plural(one=Minute,other=Minutes) Ago"), Minutes, Seconds);
		}

		if (Seconds >= 1)
		{
			return FText::Format(LOCTEXT("Secs", "{0} {0}|plural(one=Second,other=Seconds) Ago"), Seconds);
		}
		return LOCTEXT("Now", "Now");
	}

	static FText FormatTime(const FDateTime& Time, ETimeFormat TimeFormat, const FDateTime* CurrTime = nullptr)
	{
		return TimeFormat == ETimeFormat::Relative
			? FormatRelativeTime(Time, CurrTime)
			: FText::AsDateTime(Time);
	}

	static TSharedRef<SComboButton> CreateViewOptionsComboButton(FOnGetContent GetMenuContentDelegate)
	{
		return SNew(SComboButton)
			.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButtonWithIcon")
			.OnGetMenuContent(MoveTemp(GetMenuContentDelegate))
			.HasDownArrow(false)
			.ButtonContent()
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
			];
	}
};

#undef LOCTEXT_NAMESPACE
