// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertHeaderRowUtils.generated.h"

class FMenuBuilder;
class SHeaderRow;
class SWidget;

USTRUCT()
struct FColumnVisibilitySnapshot
{
	GENERATED_BODY()

	UPROPERTY()
	FString Snapshot;
};

namespace UE::ConcertSharedSlate
{
	DECLARE_DELEGATE_OneParam(FSaveColumnVisibilitySnapshot, const FColumnVisibilitySnapshot& /*Snapshot*/);

	/** Util function that can be directly fed into SListView::OnContextMenuOpening */
	CONCERTSHAREDSLATE_API TSharedRef<SWidget> MakeTableContextMenu(const TSharedRef<SHeaderRow>& HeaderRow, TMap<FName, bool> ColumnsVisibleByDefault = {}, bool bDefaultVisibility = true);

	/**
	 * Adds an entry for "Show all", "Hide all", and "Restore column visibility".
	 */
	CONCERTSHAREDSLATE_API void AddDefaultControlEntries(const TSharedRef<SHeaderRow>& HeaderRow, FMenuBuilder& MenuBuilder, TMap<FName, bool> ColumnsVisibleByDefault, bool bDefaultVisibility = true);

	/**
	 * Inspects the hidden rows on the header row and an entry for showing each hidden column.
	 * @param HeaderRow To retrieve column information
	 * @param MenuBuilder Where to add the menu entries
	 */
	CONCERTSHAREDSLATE_API void AddEntriesForShowingHiddenRows(const TSharedRef<SHeaderRow>& HeaderRow, FMenuBuilder& MenuBuilder);

	/** Exports the visibility state of each column so it can be restore later */
	CONCERTSHAREDSLATE_API FColumnVisibilitySnapshot SnapshotColumnVisibilityState(const TSharedRef<SHeaderRow>& HeaderRow);
	/** Restores the column visibilities from an exported state */
	CONCERTSHAREDSLATE_API void RestoreColumnVisibilityState(const TSharedRef<SHeaderRow>& HeaderRow, const FColumnVisibilitySnapshot& Snapshot);

	/** Helper for temporarily setting column visibilities to a certain setting. */
	class FColumnVisibilityTransaction
	{
		TMap<FName, bool> SavedColumnVisibilities;
		TMap<FName, bool> OverridenColumnVisibilities;
		TWeakPtr<SHeaderRow> HeaderRow;
	public:
		
		FColumnVisibilityTransaction(TWeakPtr<SHeaderRow> HeaderRow = nullptr);
		FColumnVisibilityTransaction& SetHeaderRow(TWeakPtr<SHeaderRow> InHeaderRow);

		FColumnVisibilityTransaction& SaveVisibilityAndSet(FName ColumnId, bool bShouldBeVisible);
		/** @param bOnlyResetIfNotOverriden Only reset to the old visibilities if none has changed from the saved state (i.e. user changed it during the operation) */
		void ResetToSavedVisibilities(bool bOnlyResetIfNotOverriden = true);
	};
};

