// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

/**
 * Interface for the Concert Sync UI module.
 */
class IConcertSharedSlateModule : public IModuleInterface
{
public:
	/** Get the ConcertSyncUI module */
	static IConcertSharedSlateModule& Get()
	{
		static const FName ModuleName = TEXT("ConcertSharedSlate");
		return FModuleManager::Get().GetModuleChecked<IConcertSharedSlateModule>(ModuleName);
	}
};
