// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EBreakBehavior.h"

#include "Containers/Array.h"
#include "Internationalization/Text.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"

namespace UE::ConcertSharedSlate
{
	enum class ESourceType : uint8
	{
		/**
		 * Display EnumerateSelectableItems in a drop-down or menu list. Each entry is a button that closes the menu.
		 * Example: User wants to search from a list of actors.
		 */
		ShowAsList,
		/**
		 * When this option is clicked, instantly add all of EnumerateSelectableItems.
		 * Example: User wants to add all actors selected in the world outliner.
		 */
		AddOnClick,
		/**
		 * Display EnumerateSelectableItems in a drop-down or menu list.
		 *
		 * Each entry contains a checkbox:
		 * - FSourceModelBuilders::OnItemsSelected is called when the checkbox is toggled (since you provide IsItemSelected, you can determine whether to add or remove the items).
		 * - FSourceModelBuilders::IsItemSelected determines whether the checkbox is checked
		 * 
		 * Example: User wants to search from a list of actors.
		 */
		ShowAsToggleButtonList,
	};

	struct FBaseDisplayInfo
	{
		/** What the model should be displayed as in UI. */
		FText Label;
		/** Optional tooltip when the model is hovered. */
		FText ToolTip;

		/** Optional icon to display entry as. */
		FSlateIcon Icon;
	};
	
	struct FSourceDisplayInfo : FBaseDisplayInfo
	{
		/** Determines what happens when the option is clicked / hovered. */
		ESourceType SourceType;
	};
	
	/**
	 * Defines the source for of item, e.g. objects.
	 *
	 * Imagine you have a list view displaying a list of objects. Over it is a "Add Object" combo button.
	 * In the button's menu, you'd find e.g. "Add from World", "Add from Selection", "Import from File", etc.
	 * Each of of these sub-menu entries is an item source and corresponds to an IItemSourceModel.
	 * @see IObjectSelectionSourceModel and FEditorObjectSelectionSourceModel for an example how sources may be used.
	 *
	 * Models may also be useful for building context menus entries. When you right-click an item, you may want to add
	 * more items based on the object clicked, e.g. add the components of a right-clicked actor.
	 * @see FEditorObjectSelectionSourceModel::FContextMenuSources and FEditorObjectSelectionSourceModel::GetContextMenuOptions
	 *
	 * @see FSourceModelBuilders for building UI from models, e.g. the above mentioned combo button or context menus.
	 */
	template<typename TItemType>
	class IItemSourceModel : public TSharedFromThis<IItemSourceModel<TItemType>>
	{
	public:

		/** Some data for displaying this source in UI. */
		virtual FSourceDisplayInfo GetDisplayInfo() const = 0;

		/** @return The number of selectable objects. */
		UE_DEPRECATED(5.5, "No longer in use")
		virtual uint32 GetNumSelectableItems() const { return 0; }
		
		/** Enumerates ALL objects from this source. This can be a LOT of objects (e.g. 2000 actors). Intended to be placed in a searchable drop-down menu. */
		virtual void EnumerateSelectableItems(TFunctionRef<EBreakBehavior(const TItemType& SelectableOption)> Delegate) const = 0;

		/** @return Whether there are any options. */
		virtual bool HasOptions() const
		{
			bool bHasAnItem = false;
			EnumerateSelectableItems([&bHasAnItem](const auto&){ bHasAnItem = true; return EBreakBehavior::Break; });
			return bHasAnItem;
		}
		
		/** Util that converts EnumerateSelectableItems into an array. */
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		TArray<TItemType> GetSelectableItems()
		{
			TArray<TItemType> Result;
			EnumerateSelectableItems([&Result](const TItemType& SelectableOption)
			{
				Result.Add(SelectableOption);
				return EBreakBehavior::Continue;
			});
			return Result;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		virtual ~IItemSourceModel() = default;
	};
}
