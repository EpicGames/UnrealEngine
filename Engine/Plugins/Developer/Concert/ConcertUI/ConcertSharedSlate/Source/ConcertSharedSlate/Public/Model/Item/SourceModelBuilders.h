// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Model/Item/IItemSourceModel.h"
#include "Model/Item/SourceSelectionCategory.h"

#include "Algo/Sort.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Internationalization/Text.h"
#include "SPositiveActionButton.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"

class FBaseMenuBuilder;

#define LOCTEXT_NAMESPACE "FSourceModelBuilders"

/**
 * These functions build widgets for selecting options.
 * TSourceSelectionCategory is an entry in a combo button menu; it can contain subcategories.
 * Categories contain IItemSourceModels which are a source of options.
 */
namespace UE::ConcertSharedSlate
{
	enum class EItemPickerFlags
	{
		None,
		/** If the source type is ESourceType::ShowAsList, instead of creating a menu entry that spawns a submenu (default behavior) just place the source's items in the same menu. */
		DisplayOptionListInline = 1 << 0
	};
	ENUM_CLASS_FLAGS(EItemPickerFlags);
	
	/**
	 * Utility for creating widgets of IItemSourceModel and categories thereof (TSourceSelectionCategory).
	 */
	template<typename TItemType>
	struct FSourceModelBuilders
	{
		DECLARE_DELEGATE_OneParam(FOnItemsSelected, TArray<TItemType>);
		DECLARE_DELEGATE_RetVal_OneParam(bool, FIsItemSelected, const TItemType& Item);
		DECLARE_DELEGATE_RetVal_OneParam(FString, FGetItemDisplayString, const TItemType& Item);
		DECLARE_DELEGATE_RetVal_OneParam(FSlateIcon, FGetItemIcon, const TItemType& Item);

		struct FItemPickerArgs
		{
			/** Required. Called when items are picked by the widget. */
			FOnItemsSelected OnItemsSelected;
			
			/** Required. Gets the display string for the item. */
			FGetItemDisplayString GetItemDisplayString;

			/** Optional. Gets the icon to display for the item. */
			FGetItemIcon GetItemIcon;

			/** Optional. Used primarily to filter out objects that are already selected. */
			FIsItemSelected IsItemSelected;

			/** Optional. Whether the UI is enabled. */
			TAttribute<bool> IsEnabledAttribute;
			/** Optional. If IsEnabled returns true, this tooltip is displayed on the relevant UI. */
			TAttribute<FText> DisabledToolTipAttribute;

			/** Special flags for altering default behavior. */
			EItemPickerFlags Flags;

			FItemPickerArgs(
				FOnItemsSelected OnObjectsSelected,
				FGetItemDisplayString GetItemDisplayString,
				FGetItemIcon GetItemIcon = {},
				FIsItemSelected IsItemSelected = {},
				TAttribute<bool> IsEnabledAttribute = {},
				TAttribute<FText> DisabledToolTipAttribute = {},
				EItemPickerFlags Flags = EItemPickerFlags::None
				)
				: OnItemsSelected(MoveTemp(OnObjectsSelected))
				, GetItemDisplayString(MoveTemp(GetItemDisplayString))
				, GetItemIcon(MoveTemp(GetItemIcon))
				, IsItemSelected(MoveTemp(IsItemSelected))
				, IsEnabledAttribute(MoveTemp(IsEnabledAttribute))
				, DisabledToolTipAttribute(MoveTemp(DisabledToolTipAttribute))
				, Flags(Flags)
			{
				check(this->OnItemsSelected.IsBound() && this->GetItemDisplayString.IsBound());
			}
		};
		
		/** Builds a combo button where each entry corresponds to one IItemSourceModel. If there is only one option, the option is inlined by calling MakeStandaloneWidget. */
		static TSharedRef<SWidget> BuildCategory(const TSourceSelectionCategory<TItemType>& Category, const FItemPickerArgs& Args);
	
		/**
		 * Creates a combo button inlining the source model. 
		 *
		 * This is the inlined case of when a category only contains one item source.
		 * Used by BuildCategory.
		 */
		static TSharedRef<SWidget> MakeStandaloneWidget(TSharedRef<IItemSourceModel<TItemType>> Source, FItemPickerArgs Args, FSlateIcon Icon);

		/** Adds the option to a context-menu */
		static void AddOptionToMenu(TSharedRef<IItemSourceModel<TItemType>> Option, FItemPickerArgs Args, FBaseMenuBuilder& MenuBuilder);

	private:
		
		static void FillSearchableSubmenu(const TSharedRef<IItemSourceModel<TItemType>>& Source, const FItemPickerArgs& Args, FBaseMenuBuilder& MenuBuilder);
		
		static TSharedRef<SWidget> BuildMenu(const TArray<TAttribute<TSharedPtr<IItemSourceModel<TItemType>>>>& Options, TArray<TSourceSelectionCategory<TItemType>> SubCategories, const FItemPickerArgs& Args);
		static bool NeedsSeparationFromOtherItems(const TSharedRef<IItemSourceModel<TItemType>>& Option, const FItemPickerArgs& Args);
		
		static void AddSubCategoryToMenu(const TSourceSelectionCategory<TItemType>& Category, const FItemPickerArgs& Args, FBaseMenuBuilder& MenuBuilder);
	};
}

/************************* Implementation *************************/

namespace UE::ConcertSharedSlate
{
	/** Builds a combo button where each entry corresponds to one IItemSourceModel. If there is only one option, the option is inlined by calling MakeStandaloneWidget. */
	template<typename TItemType>
	TSharedRef<SWidget> FSourceModelBuilders<TItemType>::BuildCategory(const TSourceSelectionCategory<TItemType>& Category, const FItemPickerArgs& Args)
	{
		if (!ensure(!Category.Options.IsEmpty()) || !ensure(Args.OnItemsSelected.IsBound()))
		{
			return SNullWidget::NullWidget;
		}
		
		if (Category.Options.Num() == 1 && Category.SubCategories.Num() == 0)
		{
			const TSharedPtr<IItemSourceModel<TItemType>> Model = Category.Options[0].Get();
			return ensure(Model) ? MakeStandaloneWidget(Model.ToSharedRef(), Args, Category.DisplayInfo.Icon) : SNullWidget::NullWidget;
		}

		return SNew(SPositiveActionButton)
			.Text(Category.DisplayInfo.Label)
			.ToolTipText_Lambda([ToolTipText = Category.DisplayInfo.ToolTip, DisabledToolTipAttribute = Args.DisabledToolTipAttribute, IsEnabled = Args.IsEnabledAttribute]()
			{
				const bool bIsEnabled = !(IsEnabled.IsBound() || IsEnabled.IsSet()) || IsEnabled.Get();
				const bool bHasDisabledText = DisabledToolTipAttribute.IsBound() || DisabledToolTipAttribute.IsSet();
				return bIsEnabled
					? ToolTipText
					: bHasDisabledText ? DisabledToolTipAttribute.Get() : FText::GetEmpty();
			})
			.Icon(Category.DisplayInfo.Icon.IsSet() ? Category.DisplayInfo.Icon.GetOptionalIcon() : FAppStyle::Get().GetBrush("Icons.Plus"))
			.IsEnabled_Lambda([IsEnabled = Args.IsEnabledAttribute](){ return !(IsEnabled.IsBound() || IsEnabled.IsSet()) || IsEnabled.Get(); })
			.OnGetMenuContent_Lambda([Options = Category.Options, SubCategories = Category.SubCategories, Args]()
			{
				return BuildMenu(Options, SubCategories, Args);
			});
	}
	
	/** Creates a widget that displays the IItemSourceModel according to its specifications. Mainly to be used by BuildCategory. */
	template<typename TItemType>
	TSharedRef<SWidget> FSourceModelBuilders<TItemType>::MakeStandaloneWidget(TSharedRef<IItemSourceModel<TItemType>> Source, FItemPickerArgs Args, FSlateIcon Icon)
	{
		const FSourceDisplayInfo DisplayInfo = Source->GetDisplayInfo();
		const FSlateBrush* IconBrush = DisplayInfo.Icon.IsSet() ? DisplayInfo.Icon.GetOptionalIcon() : FAppStyle::Get().GetBrush("Icons.Plus");

		const auto GetToolTipText = [ToolTipText = DisplayInfo.ToolTip, DisabledToolTipAttribute = Args.DisabledToolTipAttribute, IsEnabled = Args.IsEnabledAttribute]()
		{
			const bool bIsEnabled = !(IsEnabled.IsBound() || IsEnabled.IsSet()) || IsEnabled.Get();
			const bool bHasDisabledText = DisabledToolTipAttribute.IsBound() || DisabledToolTipAttribute.IsSet();
			return bIsEnabled
				? ToolTipText
				: bHasDisabledText ? DisabledToolTipAttribute.Get() : FText::GetEmpty();
		};
		
		switch (DisplayInfo.SourceType)
		{
		case ESourceType::ShowAsToggleButtonList: [[fallthrough]];
		case ESourceType::ShowAsList:
			return SNew(SPositiveActionButton)
				.Text(DisplayInfo.Label)
				.ToolTipText_Lambda(GetToolTipText)
				.Icon(IconBrush)
				.IsEnabled_Lambda([IsEnabled = Args.IsEnabledAttribute](){ return !(IsEnabled.IsBound() || IsEnabled.IsSet()) || IsEnabled.Get(); })
				.OnGetMenuContent_Lambda([Source = MoveTemp(Source), Args = MoveTemp(Args)]()
				{
					FMenuBuilder MenuBuilder(true, nullptr);
					FillSearchableSubmenu(Source, Args, MenuBuilder);
					return MenuBuilder.MakeWidget();
				});
		case ESourceType::AddOnClick: 
			return SNew(SPositiveActionButton)
				.Text(DisplayInfo.Label)
				.ToolTipText_Lambda(GetToolTipText)
				.Icon(IconBrush)
				.IsEnabled_Lambda([IsEnabled = Args.IsEnabledAttribute](){ return !(IsEnabled.IsBound() || IsEnabled.IsSet()) || IsEnabled.Get(); })
				.OnClicked_Lambda([Source = MoveTemp(Source), AddObjectsDelegate = Args.OnItemsSelected]()
				{
					AddObjectsDelegate.Execute(Source->GetSelectableItems());
					return FReply::Handled();
				});
		default:
			checkNoEntry();
			return SNullWidget::NullWidget;
		}
	}

	/** Adds the option to a context-menu */
	template<typename TItemType>
	void FSourceModelBuilders<TItemType>::AddOptionToMenu(TSharedRef<IItemSourceModel<TItemType>> Option, FItemPickerArgs Args, FBaseMenuBuilder& MenuBuilder)
	{
		const FSourceDisplayInfo DisplayInfo = Option->GetDisplayInfo();
		switch (DisplayInfo.SourceType)
		{
		case ESourceType::ShowAsList: [[fallthrough]];
		case ESourceType::ShowAsToggleButtonList:
			{
				if (EnumHasAnyFlags(Args.Flags, EItemPickerFlags::DisplayOptionListInline))
				{
					FillSearchableSubmenu(Option, Args, MenuBuilder);
				}
				else
				{
					FMenuEntryParams ShowAsListParams;
					ShowAsListParams.LabelOverride = DisplayInfo.Label;
					ShowAsListParams.ToolTipOverride = DisplayInfo.ToolTip;
					ShowAsListParams.IconOverride = DisplayInfo.Icon;
					ShowAsListParams.bIsSubMenu = true;
					// Dummy is needed to avoid assert
					ShowAsListParams.DirectActions = {
						FExecuteAction::CreateLambda([](){})
						};
					ShowAsListParams.MenuBuilder.BindLambda([Source = MoveTemp(Option), Args = MoveTemp(Args)]()
					{
						FMenuBuilder MenuBuilder(true, nullptr);
						FillSearchableSubmenu(Source, Args, MenuBuilder);
						return MenuBuilder.MakeWidget();
					});
				
					MenuBuilder.AddMenuEntry(ShowAsListParams);
				}
				break;
			}
		
		case ESourceType::AddOnClick:
			{
				FMenuEntryParams AddOnClickParams;
				AddOnClickParams.LabelOverride = DisplayInfo.Label;
				AddOnClickParams.ToolTipOverride = DisplayInfo.ToolTip;
				AddOnClickParams.IconOverride = DisplayInfo.Icon;
				AddOnClickParams.UserInterfaceActionType = EUserInterfaceActionType::Button; 
				AddOnClickParams.DirectActions = {
					FExecuteAction::CreateLambda([Option, Args = MoveTemp(Args)](){ Args.OnItemsSelected.Execute(Option->GetSelectableItems()); }),
					FCanExecuteAction::CreateLambda([Option](){ return Option->HasOptions(); })
				};
				
				MenuBuilder.AddMenuEntry(AddOnClickParams);
				break;
			}
			
		default:
			checkNoEntry();
		}
	}
	
	template<typename TItemType>
	void FSourceModelBuilders<TItemType>::FillSearchableSubmenu(const TSharedRef<IItemSourceModel<TItemType>>& Source, const FItemPickerArgs& Args, FBaseMenuBuilder& MenuBuilder)
	{
		TArray<TItemType> SortedItems = Source->GetSelectableItems();
		Algo::Sort(SortedItems, [&Args](const TItemType& Left, const TItemType& Right)
		{
			return Args.GetItemDisplayString.Execute(Left) < Args.GetItemDisplayString.Execute(Right); 
		});
		
		// Add the objects to the MenuBuilder
		const ESourceType SourceType = Source->GetDisplayInfo().SourceType;
		const bool bUseButton = SourceType == ESourceType::ShowAsList;
		
		for (const TItemType& Item : SortedItems)
		{
			// Gray out entries that will not do anything.
			const FCanExecuteAction CanExecuteAction = bUseButton
				? FCanExecuteAction::CreateLambda([Item, IsItemSelected = Args.IsItemSelected](){ return !IsItemSelected.IsBound() || !IsItemSelected.Execute(Item); })
				: FCanExecuteAction::CreateLambda([](){ return true; });
			const FIsActionChecked IsActionChecked = bUseButton
				? FIsActionChecked()
				: FIsActionChecked::CreateLambda([Item, IsItemSelected = Args.IsItemSelected](){ return IsItemSelected.IsBound() && IsItemSelected.Execute(Item); });
			MenuBuilder.AddMenuEntry(
				FText::FromString(Args.GetItemDisplayString.Execute(Item)),
				TAttribute<FText>(TAttribute<FText>::CreateLambda([CanExecuteAction](){ return CanExecuteAction.Execute() ? LOCTEXT("ItemSelected", "Item is already selected") : FText::GetEmpty(); })),
				Args.GetItemIcon.IsBound() ? Args.GetItemIcon.Execute(Item) : FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([Item, SelectItemDelegate = Args.OnItemsSelected]()
					{
						SelectItemDelegate.Execute({ Item });
					}),
					CanExecuteAction,
					IsActionChecked
					),
				NAME_None,
				bUseButton ? EUserInterfaceActionType::Button : EUserInterfaceActionType::ToggleButton
				);
		}
	}

	template<typename TItemType>
	TSharedRef<SWidget> FSourceModelBuilders<TItemType>::BuildMenu(const TArray<TAttribute<TSharedPtr<IItemSourceModel<TItemType>>>>& Options, TArray<TSourceSelectionCategory<TItemType>> SubCategories, const FItemPickerArgs& Args)
	{
		FMenuBuilder MenuBuilder(true, nullptr);

		// Sources first...
		bool bPreviousSourceNeededSeparator = false;
		for (int32 i = 0; i < Options.Num(); ++i)
		{
			const TAttribute<TSharedPtr<IItemSourceModel<TItemType>>>& Source = Options[i];
			const TSharedPtr<IItemSourceModel<TItemType>> Model = Source.Get();
			if (ensure(Model))
			{
				const TSharedRef<IItemSourceModel<TItemType>> ModelRef = Model.ToSharedRef();

				// E.g. if the source is a list of (many) inlined items, they are separated
				bPreviousSourceNeededSeparator = NeedsSeparationFromOtherItems(ModelRef, Args);
				const bool bIsNotFirstItem = i > 0;
				if (bPreviousSourceNeededSeparator && bIsNotFirstItem)
				{
					MenuBuilder.AddSeparator();
				}
				
				AddOptionToMenu(ModelRef, Args, MenuBuilder);
				
				const bool bIsNotLastItem = i < Options.Num() - 1;
				if (bPreviousSourceNeededSeparator && bIsNotLastItem)
				{
					MenuBuilder.AddSeparator();
				}
			}
		}
			
		// ... then the sub-categories sorted by their names
		SubCategories.Sort([](const TSourceSelectionCategory<TItemType>& Left, const TSourceSelectionCategory<TItemType>& Right)
		{
			// Is this slow? May want to cache ToString(). The number of items is so low it will hopefully have no noticeable effect.
			return Left.DisplayInfo.Label.ToString() < Right.DisplayInfo.Label.ToString();
		});
		if (!SubCategories.IsEmpty() && bPreviousSourceNeededSeparator)
		{
			MenuBuilder.AddSeparator();
		}
		for (const TSourceSelectionCategory<TItemType>& SubCategory : SubCategories)
		{
			AddSubCategoryToMenu(SubCategory, Args, MenuBuilder);
		}
			
		return MenuBuilder.MakeWidget();
	}

	template <typename TItemType>
	bool FSourceModelBuilders<TItemType>::NeedsSeparationFromOtherItems(const TSharedRef<IItemSourceModel<TItemType>>& Option, const FItemPickerArgs& Args)
	{
		const bool bIsInlinedItemList = Option->GetDisplayInfo().SourceType == ESourceType::ShowAsList
			&& EnumHasAnyFlags(Args.Flags, EItemPickerFlags::DisplayOptionListInline);
		return bIsInlinedItemList;
	}

	template<typename TItemType>
	void FSourceModelBuilders<TItemType>::AddSubCategoryToMenu(const TSourceSelectionCategory<TItemType>& Category, const FItemPickerArgs& Args, FBaseMenuBuilder& MenuBuilder)
	{
		FMenuEntryParams MenuEntry;
		MenuEntry.LabelOverride = Category.DisplayInfo.Label;
		MenuEntry.ToolTipOverride = Category.DisplayInfo.ToolTip;
		MenuEntry.bIsSubMenu = true;
		// Dummy is needed to avoid assert
		MenuEntry.DirectActions = {
			FExecuteAction::CreateLambda([](){})
			};
		MenuEntry.IconOverride = Category.DisplayInfo.Icon;
		MenuEntry.MenuBuilder.BindLambda([Options = Category.Options, SubCategories = Category.SubCategories, Args]() mutable
		{
			return BuildMenu(Options, MoveTemp(SubCategories), Args);
		});

		MenuBuilder.AddMenuEntry(MenuEntry);
	}
}

#undef LOCTEXT_NAMESPACE