// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IItemSourceModel.h"

#include "Containers/Array.h"
#include "Misc/Attribute.h"

namespace UE::ConcertSharedSlate
{
	/**
	 * A category groups IItemSourceModel together, e.g. a category "Add actor" could have options like
	 *	- selected in outliner
	 *	- drop-down list of actors in the world
	 *
	 *	Usually a category is displayed by a combo button with DisplayInfo being applied to it.
	 *	However, if Options.Num() == 1, then for better UX the button can be "inlined" with a widget visualizing that option,
	 *	e.g. a button that just invokes the single option and its IObjectSourceModel::GetDisplayInfo is applied to the button.
	 */
	template<typename TSelectionType>
	struct TSourceSelectionCategory
	{
		/** Data to display about this category. */
		FBaseDisplayInfo DisplayInfo;
		
		/** The options of this category. TAttribute to allow lazy instantiation (e.g. only when sub-menu is opened by user). */
		TArray<TAttribute<TSharedPtr<IItemSourceModel<TSelectionType>>>> Options;

		/** Sub-categories to show in the context menu. Example: "Add component" is subcategory with the items being FComponentFromActorSource_Root. */
		TArray<TSourceSelectionCategory> SubCategories;
	};
}
