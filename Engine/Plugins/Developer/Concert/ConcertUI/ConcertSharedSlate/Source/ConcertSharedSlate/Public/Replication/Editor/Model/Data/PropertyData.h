// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Data/ConcertPropertySelection.h"

#include "Containers/Array.h"
#include "UObject/SoftObjectPtr.h"

namespace UE::ConcertSharedSlate
{
	/** Instanced for each property row in IPropertyTreeView.*/
	class FPropertyData
	{
	public:
		
		FPropertyData(TArray<TSoftObjectPtr<>> ContextObjects, FSoftClassPath OwningClass, FConcertPropertyChain Object)
			: ContextObjects(MoveTemp(ContextObjects))
			, OwningClassPtr(MoveTemp(OwningClass))
			, Property(MoveTemp(Object))
		{}
		
		const TArray<TSoftObjectPtr<>>& GetContextObjects() const { return ContextObjects; }
		const FConcertPropertyChain& GetProperty() const { return Property; }
		const TSoftClassPtr<>& GetOwningClassPtr() const { return OwningClassPtr; }

	private:

		/**
		 * The objects for which the properties are being displayed.
		 *
		 * This usually has only 1 entry.
		 * This has multiple elements in the case of multi-edit (i.e. when the user clicks multiple, compatible actors in the top-view).
		 * For example, for multi-edit this could contain ActorA->StaticMeshComponent0 and ActorB->StaticMeshComponent0.
		 */
		TArray<TSoftObjectPtr<>> ContextObjects;
		
		/** The class with which the FProperty can be determined. */
		TSoftClassPtr<> OwningClassPtr;
		/**
		 * The property to be replicated.
		 *
		 * On the servers, this will usually not resolve to anything.
		 * This was promoted to be TSoftObjectPtr so that any UI that resolves this object path automatically caches it.
		 * In certain operations this can improve performance: e.g. when fully rebuilding the property tree, this saved 35% performance.
		 */
		FConcertPropertyChain Property;
	};
}