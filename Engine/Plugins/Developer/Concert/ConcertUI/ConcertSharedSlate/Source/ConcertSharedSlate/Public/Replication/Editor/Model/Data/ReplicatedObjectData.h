// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"

namespace UE::ConcertSharedSlate
{
	/** Instanced for each object row in stream viewer / editor UI.*/
	class FReplicatedObjectData
	{
	public:

		FReplicatedObjectData(FSoftObjectPath ObjectPath)
			: ObjectPtr(MoveTemp(ObjectPath))
		{}
		FReplicatedObjectData(TSoftObjectPtr<> Object)
			: ObjectPtr(MoveTemp(Object))
		{}
		
		const FSoftObjectPath& GetObjectPath() const { return ObjectPtr.GetUniqueID(); }
		const TSoftObjectPtr<>& GetObjectPtr() const { return ObjectPtr; }

	private:

		/**
		 * The replicated object.
		 *
		 * On the servers, this will usually not resolve to anything.
		 * This was promoted to be TSoftObjectPtr so that any UI that resolves this object path automatically caches it.
		 * In certain operations this can improve performance: e.g. when fully rebuilding the property tree, this saved 35% performance.
		 */
		TSoftObjectPtr<> ObjectPtr;
	};
}