// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"

class UObject;
struct FConcertPropertyChain;

namespace UE::ConcertSharedSlate
{
	class IStreamExtensionContext;

	/** When an object is added to IEditableReplicationStreamModel, these functions are called to allow adding additional objects and properties. */
	class IStreamExtender
	{
	public:

		/**
		 * Allows extending the stream when objects are added.
		 * This function is called recursively for any additional objects you add.
		 * 
		 * @param ExtendedObject Object that was added
		 * @param Context Use for adding properties to ExtendedObject or to additional objects.
		 */
		virtual void ExtendStream(UObject& ExtendedObject, IStreamExtensionContext& Context) = 0;

		virtual ~IStreamExtender() = default;
	};
}
