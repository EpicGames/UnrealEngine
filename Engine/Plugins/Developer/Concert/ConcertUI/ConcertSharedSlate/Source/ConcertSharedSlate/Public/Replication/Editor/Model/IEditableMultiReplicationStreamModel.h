// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Misc/EBreakBehavior.h"
#include "IEditableReplicationStreamModel.h"
#include "IMultiReplicationStreamModel.h"
#include "IReplicationStreamModel.h"
#include "Templates/SharedPointer.h"

namespace UE::ConcertSharedSlate
{
	class IEditableReplicationStreamModel;
	
	/**
	 * Model for UI editing multiple streams at the same time.
	 *
	 * Some streams can be read but not edited:
	 * - the set of readable streams includes the set of editable streams, but
	 * - the number of editable streams can be less than or equal to the number of read-only streams.
	 */
	class IEditableMultiReplicationStreamModel : public IMultiReplicationStreamModel
	{
	public:

		/**
		 * @return Gets the streams that can be written to.
		 * 
		 * @note There may be streams that are readable but not editable.
		 * Formally, the constraint is GetReadableStreams().Includes(GetEditableStreams()) and GetEditableStreams().Num() <= GetReadOnlyStreams().Num().
		 */
		virtual TSet<TSharedRef<IEditableReplicationStreamModel>> GetEditableStreams() const = 0;

		/** Util for iterating through both read-only and editable streams. */
		void ForEachStream(TFunctionRef<EBreakBehavior(const TSharedRef<IReplicationStreamModel>& Model)> Callback) const
		{
			const TSet<TSharedRef<IReplicationStreamModel>> ReadableModels = GetReadOnlyStreams();
			for (const TSharedRef<IReplicationStreamModel>& Model : GetReadOnlyStreams())
			{
				if (Callback(Model) == EBreakBehavior::Break)
				{
					return;
				}
			}
			
			for (const TSharedRef<IEditableReplicationStreamModel>& Model : GetEditableStreams())
			{
				const TSharedRef<IReplicationStreamModel> CastModel = StaticCastSharedRef<IReplicationStreamModel, IEditableReplicationStreamModel>(Model);
				if (!ReadableModels.Contains(CastModel) // Already iterated it if it is contained
					&& Callback(CastModel) == EBreakBehavior::Break)
				{
					return;
				}
			}
		}
	};
}
