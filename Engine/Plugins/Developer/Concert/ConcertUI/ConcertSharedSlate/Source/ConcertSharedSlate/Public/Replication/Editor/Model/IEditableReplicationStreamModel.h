// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IReplicationStreamModel.h"

#include "Containers/ContainersFwd.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Delegates/DelegateCombinations.h"

class UObject;
struct FConcertPropertyChain;

namespace UE::ConcertSharedSlate
{
	enum class EReplicatedObjectChangeReason : uint8
	{
		/** A function call on the model changed the data. */
		ChangedDirectly,
		
		/**
		 * An external change, such as transaction, changed the data out from under us.
		 * The AddedObjects and RemovedObjects fields are empty because we don't know exactly what changed, if anything.
		 */
		ExternalChange
	};
	
	/**
	 * Abstracts the concept of mapping objects to properties. This allows writing.
	 * 
	 * Models may not always be writable. When editing a UAsset, it will be writable. However, if we join a multi-user
	 * session we do not want to edit the objects nor properties - only read.
	 */
	class IEditableReplicationStreamModel : public IReplicationStreamModel
	{
	public:

		/** Adds the objects to the mapping */
		virtual void AddObjects(TConstArrayView<UObject*> Objects) = 0;
		/** Removes the objects from the mapping */
		virtual void RemoveObjects(TConstArrayView<FSoftObjectPath> Objects) = 0;

		/** Adds these properties to the object's list of selected properties. */
		virtual void AddProperties(const FSoftObjectPath& Object, TConstArrayView<FConcertPropertyChain> Properties) = 0;
		/** Removes these properties from the object's list of selected properties. */
		virtual void RemoveProperties(const FSoftObjectPath& Object, TConstArrayView<FConcertPropertyChain> Properties) = 0;
		
		/** Called when the object list changes. AddedObjects and RemovedObjects are empty if and only if ChangeReason == EReplicatedObjectChangeReason::Transacted. */
		DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnObjectsChanged, TConstArrayView<UObject*> AddedObjects, TConstArrayView<FSoftObjectPath> RemovedObjects, EReplicatedObjectChangeReason ChangeReason);
		virtual FOnObjectsChanged& OnObjectsChanged() = 0;

		/** Called when the property list of some object changes. If the properties are removed because the object is removed outright, FOnObjectsChanged is called instead.  */
		DECLARE_MULTICAST_DELEGATE(FOnPropertiesChanged);
		virtual FOnPropertiesChanged& OnPropertiesChanged() = 0;

		/** Clear all objects in the model */
		void Clear() { RemoveObjects(GetReplicatedObjects().Array()); }
	};
}
