// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"

namespace UE::ConcertSharedSlate
{
	class IReplicationStreamModel;
	
	/** Model for UI displaying multiple streams at the same time. */
	class IMultiReplicationStreamModel : public TSharedFromThis<IMultiReplicationStreamModel>
	{
	public:

		/** @return Streams in this model that can only be read from. */
		virtual TSet<TSharedRef<IReplicationStreamModel>> GetReadOnlyStreams() const = 0;

		DECLARE_MULTICAST_DELEGATE_OneParam(FOnStreamExternallyChanged, TSharedRef<IReplicationStreamModel> /* ChangedStream */);
		/** Called when a read-only stream has changed. */
		virtual FOnStreamExternallyChanged& OnStreamExternallyChanged() = 0;
		
		DECLARE_MULTICAST_DELEGATE(FOnStreamSetChanged);
		/** Broadcasts when the result of GetStreams has changed. */
		virtual FOnStreamSetChanged& OnStreamSetChanged() = 0;

		virtual ~IMultiReplicationStreamModel() = default;
	};
}
