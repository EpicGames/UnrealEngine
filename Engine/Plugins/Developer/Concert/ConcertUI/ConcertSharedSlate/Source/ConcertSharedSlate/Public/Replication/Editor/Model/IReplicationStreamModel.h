// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EBreakBehavior.h"
#include "Replication/Data/ConcertPropertySelection.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

namespace UE::ConcertSharedSlate
{
	/**
	 * Abstracts the concept of mapping objects to properties.
	 * This is a read-only interface.
	 *
	 * Objects are allowed to be bound to no properties.
	 * 
	 * @see IEditableReplicationStreamModel 
	 */
	class IReplicationStreamModel : public TSharedFromThis<IReplicationStreamModel>
	{
	public:

		/** @return Gets the class of the replication object */
		virtual FSoftClassPath GetObjectClass(const FSoftObjectPath& Object) const = 0;

		/** @return Whether the objects are selected */
		virtual bool ContainsObjects(const TSet<FSoftObjectPath>& Objects) const = 0;
		/** @return Whether the properties are selected */
		virtual bool ContainsProperties(const FSoftObjectPath& Object, const TSet<FConcertPropertyChain>& Properties) const = 0;
		
		/**
		 * Iterates all objects in the mapping.
		 * @return Whether there were any mapped objects
		 */
		virtual bool ForEachReplicatedObject(TFunctionRef<EBreakBehavior(const FSoftObjectPath& Object)> Delegate) const = 0;

		/**
		 * Iterates all the properties of the given object in the mapping.
		 * @param Object The object of which to get the properties
		 * @param Delegate The callback
		 */
		virtual bool ForEachProperty(const FSoftObjectPath& Object, TFunctionRef<EBreakBehavior(const FConcertPropertyChain& Property)> Delegate) const = 0;

		/** Gets the number of properties assigned to Object. */
		virtual uint32 GetNumProperties(const FSoftObjectPath& Object) const = 0;

		/** @return Gets all replicated objects */
		TSet<FSoftObjectPath> GetReplicatedObjects() const
		{
			TSet<FSoftObjectPath> Result;
			ForEachReplicatedObject([&Result](const FSoftObjectPath& ObjectPath)
			{
				Result.Add(ObjectPath);
				return EBreakBehavior::Continue;
			});
			return Result;
		}
		
		/**
		 * Iterates all the direct child properties of a given property in the mapping.
		 *
		 * @param Object The object of which to get the properties
		 * @param Delegate The callback
		 * @param OptionalParent Defines of which properties to get the child properties. If the property chain is empty, gets the object's root properties.
		 *
		 * @return Whether any properties were mapped
		 */
		bool ForEachDirectChildProperty(const FSoftObjectPath& Object, TFunctionRef<EBreakBehavior(const FConcertPropertyChain& Property)> Delegate, const FConcertPropertyChain& OptionalParent = {}) const
		{
			bool bFoundAtLeastOne = false;
			ForEachProperty(Object, [&Delegate, &OptionalParent, &bFoundAtLeastOne](const FConcertPropertyChain& Property) mutable
			{
				const bool bCanCall = Property == OptionalParent;
				bFoundAtLeastOne |= bCanCall;
				return bCanCall
					? Delegate(Property)
					: EBreakBehavior::Continue;
			});
			return bFoundAtLeastOne;
		}

		/** @return Whether the given SearchedProperty is assigned to ObjectPath. */
		bool HasProperty(const FSoftObjectPath& ObjectPath, const FConcertPropertyChain& SearchedProperty) const
		{
			bool bFound = false;
			ForEachProperty(ObjectPath, [&SearchedProperty, &bFound](const FConcertPropertyChain& Property)
			{
				bFound = SearchedProperty == Property;
				return bFound ? EBreakBehavior::Break : EBreakBehavior::Continue;
			});
			return bFound;
		}

		/** @return Whether there is at least one property assigned to ObjectPath. */
		bool HasAnyPropertyAssigned(const FSoftObjectPath& ObjectPath) const
		{
			bool bFound = false;
			ForEachProperty(ObjectPath, [&bFound](const FConcertPropertyChain& Property)
			{
				bFound = true;
				return EBreakBehavior::Break;
			});
			return bFound;
		}

		/** Util for getting properties as array */
		TSet<FConcertPropertyChain> GetAllProperties(const FSoftObjectPath& Object) const
		{
			TSet<FConcertPropertyChain> Result;
			const uint32 NumProperties = GetNumProperties(Object);
			if (NumProperties == 0)
			{
				return Result;
			}

			Result.Reserve(NumProperties);
			ForEachProperty(Object, [&Result](const FConcertPropertyChain& Property)
			{
				Result.Add(Property);
				return EBreakBehavior::Continue;
			});
			return Result;
		}
		
		/** Util for getting child properties as array */
		TSet<FName> GetChildProperties(const FSoftObjectPath& Object, const FConcertPropertyChain& OptionalParent = {}) const
		{
			TSet<FName> Result;
			ForEachDirectChildProperty(Object, [&Result](const FConcertPropertyChain& Property)
			{
				Result.Add(Property.GetLeafProperty());
				return EBreakBehavior::Continue;
			}, OptionalParent);
			return Result;
		}

		/** Iterates every subobject of Parent. */
		void ForEachSubobject(const FSoftObjectPath& Parent, TFunctionRef<EBreakBehavior(const FSoftObjectPath& Child)> Callback) const
		{
			const FString ParentPathString = Parent.ToString();
			ForEachReplicatedObject([&Parent, &ParentPathString, &Callback](const FSoftObjectPath& Object)
			{
				return Object != Parent && Object.ToString().Contains(ParentPathString)
					? Callback(Object)
					: EBreakBehavior::Continue;
			});
		}

		/** @return Gets the subobjects as an array */
		template<typename TAllocator = FDefaultAllocator>
		TArray<FSoftObjectPath, TAllocator> GetSubobjects(const FSoftObjectPath& Parent) const
		{
			TArray<FSoftObjectPath, TAllocator> Subobjects;
			ForEachSubobject(Parent, [&Subobjects](const FSoftObjectPath& Child)
			{
				Subobjects.Add(Child);
				return EBreakBehavior::Continue;
			});
			return Subobjects;
		}

		/** @return Whether the predicate holds true for any of Root's subobjects. */
		template<typename TLambda> requires std::is_invocable_r_v<bool, TLambda, const FSoftObjectPath&>
		bool AnyOfSubobjects(const FSoftObjectPath& Root, TLambda&& Lambda)
		{
			bool bResult = false;
			ForEachSubobject(
				Root,
				[&Lambda, &bResult](const FSoftObjectPath& Child)
				{
					bResult = Lambda(Child);
					return bResult ? EBreakBehavior::Break : EBreakBehavior::Continue;
				});
			return bResult;
		}
		
		virtual ~IReplicationStreamModel() = default;
	};
}
