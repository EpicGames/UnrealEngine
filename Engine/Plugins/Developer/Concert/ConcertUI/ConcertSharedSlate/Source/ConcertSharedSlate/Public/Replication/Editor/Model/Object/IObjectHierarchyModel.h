// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Misc/EBreakBehavior.h"
#include "Templates/Function.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"

#define UE_API CONCERTSHAREDSLATE_API

enum class EBreakBehavior : uint8;

namespace UE::ConcertSharedSlate
{
	/** Describes the kind of relationship a child object can have to its parent object */
	enum class EChildRelationship : uint8
	{
		/** The child is a child component */
		Component,
		/** The child is subobject but not a component */
		Subobject,

		/** The relationship is not further specified / unknown (for example in non-editor builds you may need to use this due to missing runtime info) */
		Unknown

		// Could add another one for child actor
	};

	/** Similar to EChildRelationship. Used for filtering using bit flags. */
	enum class EChildRelationshipFlags : uint8
	{
		None = 0,
		
		/** The child is a component of the parent. */
		Component = 1 << 0,
		/** The child is a subobject of the parent but no component. */
		Subobject = 1 << 1,

		/** The relationship is not further specified / unknown (for example in non-editor builds you may need to use this due to missing runtime info) */
		Unknown = 1 << 2,

		All = Component | Subobject | Unknown
	};
	ENUM_CLASS_FLAGS(EChildRelationshipFlags);
	
	/**
	 * Determines the parent-child relationship between objects.
	 * The model can be given an object and it will list all child objects, usually subobjects or child components.
	 *
	 * In editor builds, an implementation could resolve object paths to AActors, USceneComponent, etc.
	 * In programs, an implementation could be to compare all known object path strings.
	 */
	class IObjectHierarchyModel
	{
	public:
		
		/**
		 * Gets all direct children in the hierarchy.
		 *
		 * An example for what ForEachDirectChild could return in an editor build:
		 * - Actor returns its root component, non-scene components (UActorComponents), and all other direct subobjects
		 * - Scene components return their children, and all other direct subobjects
		 * - All other subobjects report all direct subobjects
		 * The hierarchy is implementation defined: the above is just an example for describing how ForEachDirectChild should operate.
		 *
		 * @param Root
		 * @param Callback
		 * @param InclusionFlags Specifies the relationships that should be included
		 */
		virtual void ForEachDirectChild(
			const TSoftObjectPtr<>& Root,
			TFunctionRef<EBreakBehavior(const TSoftObjectPtr<>& Object, EChildRelationship Relationship)> Callback,
			EChildRelationshipFlags InclusionFlags = EChildRelationshipFlags::All
			) const = 0;

		struct FParentInfo
		{
			/** The parent of the child object in the hierarchy. */
			TSoftObjectPtr<const UObject> Parent;
			/** Relationship that the child has to Parent, e.g. the child could be a component of Parent. */
			EChildRelationship Relationship;
		};
		/** Gets parent info for ChildObject, if it has a child. */
		virtual TOptional<FParentInfo> GetParentInfo(const TSoftObjectPtr<const UObject>& ChildObject) const = 0;

		/** Util for iterating all subobjects. */
		UE_API void ForEachChildRecursive(
			const TSoftObjectPtr<>& Root,
			TFunctionRef<EBreakBehavior(const TSoftObjectPtr<>& Parent, const TSoftObjectPtr<>& ChildObject, EChildRelationship Relationship)> Callback,
			EChildRelationshipFlags InclusionFlags = EChildRelationshipFlags::All
			) const;

		/** Builds a TArray containing all children, recursively. */
		template<typename TAllocator = FDefaultAllocator>
		TArray<TSoftObjectPtr<>, TAllocator> GetChildrenRecursive(const TSoftObjectPtr<>& Root, EChildRelationshipFlags InclusionFlags = EChildRelationshipFlags::All) const
		{
			TArray<TSoftObjectPtr<>, TAllocator> AllObjects;
			ForEachChildRecursive(Root,
				[&AllObjects](const TSoftObjectPtr<>&, const TSoftObjectPtr<>& ChildObject, EChildRelationship)
				{
					AllObjects.Add(ChildObject);
					return EBreakBehavior::Continue;
				}, InclusionFlags);
			return AllObjects;
		}

		virtual ~IObjectHierarchyModel() = default;
	};
}

#undef UE_API
