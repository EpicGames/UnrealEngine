// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"
#include "UObject/SoftObjectPtr.h"

class FText;

namespace UE::ConcertSharedSlate
{
	/**
	 * Abstracts the concept of getting an object's display name.
	 *
	 * In editor builds, this could use the USubobjectDataSubsystem to look up subobject names on an actor.
	 * In programs, it could just default to what it says in the object path.
	 */
	class IObjectNameModel
	{
	public:

		/** @return Gets the display name for an object */
		virtual FText GetObjectDisplayName(const TSoftObjectPtr<>& ObjectPath) const = 0;

		// In the future, we could add an event here for when an object's display name changes.

		virtual ~IObjectNameModel() = default;
	};
}
