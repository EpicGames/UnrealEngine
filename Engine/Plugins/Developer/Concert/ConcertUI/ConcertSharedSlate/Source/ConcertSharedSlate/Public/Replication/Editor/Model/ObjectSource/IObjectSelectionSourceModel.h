// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "IObjectSourceModel.h"
#include "Model/Item/SourceSelectionCategory.h"
#include "Templates/SharedPointer.h"

namespace UE::ConcertSharedSlate
{
	using FObjectSourceCategory = ConcertSharedSlate::TSourceSelectionCategory<FSelectableObjectInfo>;
	
	enum class EObjectItemValidity : uint8
	{
		/** The item is allowed */
		Valid,
		/** The referenced object no longer exists. E.g. the actor was removed. */
		DoesNotExist,
		/** Any other reason, e.g. the object was allowed in a previous version but no longer is allowed now. */
		Invalid
	};
	
	/** Decides which objects can be added to a IEditableReplicationStreamModel. */
	class IObjectSelectionSourceModel : public TSharedFromThis<IObjectSelectionSourceModel>
	{
	public:

		/** Gets the "root set" of sources. Each one should be a button next to the search bar, e.g. "Add actor". */
		virtual TArray<FObjectSourceCategory> GetRootSources() const = 0;

		/** @return The sources that should be displayed in the right-click context menu when clicked in the menu. */
		virtual TArray<TSharedRef<IObjectSourceModel>> GetContextMenuOptions(const FSoftObjectPath& Items) = 0;
		
		virtual ~IObjectSelectionSourceModel() = default;
	};
}
