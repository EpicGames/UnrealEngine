// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Model/Item/IItemSourceModel.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UObject;

namespace UE::ConcertSharedSlate
{
	/** Info about a selectable object */
	struct FSelectableObjectInfo
	{
		FSelectableObjectInfo(UObject& Object) : Object(&Object) {}
		
		TWeakObjectPtr<UObject> Object;
	};
	
	/**
	 * A specific object source, e.g. like actors, components from an actor (right-click), etc.
	 * @see IObjectSelectionSourceModel.
	 */
	using IObjectSourceModel = ConcertSharedSlate::IItemSourceModel<FSelectableObjectInfo>;
}
