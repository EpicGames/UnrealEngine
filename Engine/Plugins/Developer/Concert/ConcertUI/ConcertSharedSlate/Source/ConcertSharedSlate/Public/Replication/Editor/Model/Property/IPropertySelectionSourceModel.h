// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertySourceModel.h"
#include "Model/Item/SourceSelectionCategory.h"

#include "Misc/CoreMiscDefines.h"
#include "Templates/SharedPointer.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"

namespace UE::ConcertSharedSlate
{
	using FPropertySourceCategory UE_DEPRECATED(5.5, "No longer in use.") =
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ConcertSharedSlate::TSourceSelectionCategory<FSelectablePropertyInfo>;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	/** Decides which properties can be added to a IReplicationStreamModel. */
	class UE_DEPRECATED(5.5, "Use IPropertySourceProcessor instead.") IPropertySelectionSourceModel
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		: public TSharedFromThis<IPropertySelectionSourceModel>
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
	public:

		/** Gets the single source determining which properties can be selected. */
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		virtual TSharedRef<IPropertySourceModel> GetPropertySource(const FSoftClassPath& Class) const = 0;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		
		virtual ~IPropertySelectionSourceModel() = default;
	};
}
