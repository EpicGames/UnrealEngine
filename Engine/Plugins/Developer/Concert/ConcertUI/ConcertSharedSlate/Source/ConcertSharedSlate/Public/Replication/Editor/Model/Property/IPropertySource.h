// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EBreakBehavior.h"

#include "Templates/FunctionFwd.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"

struct FConcertPropertyChain;

namespace UE::ConcertSharedSlate
{
	/** Wraps FConcertPropertyChain so it is easier to potentially change what IPropertySource lists in the future. */
	struct FPropertyInfo
	{
		const FConcertPropertyChain& Property;
		explicit FPropertyInfo(const FConcertPropertyChain& Property) : Property(Property) {}
	};
	
	/** Lists out a bunch of properties. */
	class IPropertySource
	{
	public:

		/** Lists a bunch of properties. */
		virtual void EnumerateProperties(TFunctionRef<EBreakBehavior(const FPropertyInfo& Property)> Delegate) const = 0;
		
		virtual ~IPropertySource() = default;
	};
}
