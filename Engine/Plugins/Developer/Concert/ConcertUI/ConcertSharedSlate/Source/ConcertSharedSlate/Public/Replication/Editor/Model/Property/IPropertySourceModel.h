// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Model/Item/IItemSourceModel.h"
#include "Replication/Data/ConcertPropertySelection.h"

#include "Misc/CoreMiscDefines.h"

namespace UE::ConcertSharedSlate
{
	/** Info about a selectable property */
	struct UE_DEPRECATED(5.5, "No longer in use.") FSelectablePropertyInfo
	{
		FConcertPropertyChain Property;
	};
	
	/**
	 * A specific object source, e.g. like actors, components from an actor (right-click), etc.
	 * @see IPropertySelectionProcessor.
	 */
	using IPropertySourceModel UE_DEPRECATED(5.5, "No longer in use.") =
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ConcertSharedSlate::IItemSourceModel<FSelectablePropertyInfo>;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
