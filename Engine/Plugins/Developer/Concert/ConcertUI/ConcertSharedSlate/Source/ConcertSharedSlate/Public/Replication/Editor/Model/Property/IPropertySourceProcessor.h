// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/FunctionFwd.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"

namespace UE::ConcertSharedSlate
{
	class IPropertySource;
	
	/** Arguments for getting properties associated with an object / class. */
	struct FPropertySourceContext
	{
		/** The object for which the properties are supposed to be displayed */
		TSoftObjectPtr<> Object;
		/** The class of Object */
		FSoftClassPath Class;

		FPropertySourceContext(const TSoftObjectPtr<>& Object, const FSoftClassPath& Class)
			: Object(Object)
			, Class(Class)
		{}
	};
	
	/**
	 * Determines the properties that should be displayed for an Object / Class.
	 * 
	 * The most simple implementation is to iterate the UClass properties but there can be more advanced implementations, such as only returning
	 * properties from a user defined list.
	 */
	class IPropertySourceProcessor
	{
	public:
		
		/** Allows you to get the properties that are associated with the passed in object / class context. You are not allowed to keep a reference to Processor. */
		virtual void ProcessPropertySource(const FPropertySourceContext& Context, TFunctionRef<void(const IPropertySource& Model)> Processor) const = 0;
		
		virtual ~IPropertySourceProcessor() = default;
	};
}
