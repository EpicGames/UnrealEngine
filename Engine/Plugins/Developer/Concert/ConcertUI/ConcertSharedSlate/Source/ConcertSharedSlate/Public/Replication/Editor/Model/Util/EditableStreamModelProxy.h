// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Editor/Model/IEditableReplicationStreamModel.h"

namespace UE::ConcertSharedSlate
{
	/**
	 * Base util class for implementations that alter or extend the behavior of another IEditableReplicationStreamModel.
	 * This pattern allows chaining of behaviors.
	 */
	class FEditableStreamModelProxy : public IEditableReplicationStreamModel
	{
	public:

		FEditableStreamModelProxy(const TSharedRef<IEditableReplicationStreamModel>& InModel)
			: WrappedModel(InModel)
		{}

		//~ Begin IReplicationStreamModel Interface
		virtual FSoftClassPath GetObjectClass(const FSoftObjectPath& Object) const override { return WrappedModel->GetObjectClass(Object); }
		virtual bool ContainsObjects(const TSet<FSoftObjectPath>& Objects) const override { return WrappedModel->ContainsObjects(Objects); }
		virtual bool ContainsProperties(const FSoftObjectPath& Object, const TSet<FConcertPropertyChain>& Properties) const override { return WrappedModel->ContainsProperties(Object, Properties); }
		virtual bool ForEachReplicatedObject(TFunctionRef<EBreakBehavior(const FSoftObjectPath& Object)> Delegate) const override { return WrappedModel->ForEachReplicatedObject(Delegate); }
		virtual bool ForEachProperty(const FSoftObjectPath& Object, TFunctionRef<EBreakBehavior(const FConcertPropertyChain& Property)> Delegate) const override { return WrappedModel->ForEachProperty(Object, Delegate); }
		virtual uint32 GetNumProperties(const FSoftObjectPath& Object) const override { return WrappedModel->GetNumProperties(Object); }
		virtual FOnObjectsChanged& OnObjectsChanged() override { return WrappedModel->OnObjectsChanged(); }
		virtual FOnPropertiesChanged& OnPropertiesChanged() override { return WrappedModel->OnPropertiesChanged(); }
		//~ End IReplicationStreamModel Interface
		
		//~ Begin IEditableReplicationStreamModel Interface
		virtual void AddObjects(TConstArrayView<UObject*> Objects) override { WrappedModel->AddObjects(Objects); }
		virtual void RemoveObjects(TConstArrayView<FSoftObjectPath> Objects) override { WrappedModel->RemoveObjects(Objects); }
		virtual void AddProperties(const FSoftObjectPath& Object, TConstArrayView<FConcertPropertyChain> Properties) override { WrappedModel->AddProperties(Object, Properties); }
		virtual void RemoveProperties(const FSoftObjectPath& Object, TConstArrayView<FConcertPropertyChain> Properties) override { WrappedModel->RemoveProperties(Object, Properties); }
		//~ End IEditableReplicationStreamModel Interface

	protected:

		const TSharedRef<IEditableReplicationStreamModel>& GetWrappedModel() const { return WrappedModel; }

	private:
		
		/** The model to be transacted */
		TSharedRef<IEditableReplicationStreamModel> WrappedModel;
	};
}
