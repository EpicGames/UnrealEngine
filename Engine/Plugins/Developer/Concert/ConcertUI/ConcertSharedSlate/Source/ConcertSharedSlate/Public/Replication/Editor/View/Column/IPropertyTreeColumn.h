// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IReplicationTreeColumn.h"
#include "ReplicationColumnInfo.h"
#include "Replication/Editor/Model/Data/PropertyData.h"

namespace UE::ConcertSharedSlate
{
	/** Holds information relevant to a row. */
	struct FPropertyTreeRowContext
	{
		/** The data that is stored in the row */
		FPropertyData RowData;
	};
	
	using IPropertyTreeColumn = IReplicationTreeColumn<FPropertyTreeRowContext>;
	using FPropertyColumnEntry = TReplicationColumnEntry<FPropertyTreeRowContext>;
}
