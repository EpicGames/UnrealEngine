// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Views/SHeaderRow.h"

class FString;
class FText;

namespace UE::ConcertSharedSlate
{
	template<typename TTreeItemType>
	class IReplicationTreeColumn
	{
	public:

		struct FBuildArgs
		{
			TSharedPtr<FText> HighlightText;
			const TTreeItemType& RowItem;
		};
		
		virtual SHeaderRow::FColumn::FArguments CreateHeaderRowArgs() const = 0;

		virtual TSharedRef<SWidget> GenerateColumnWidget(const FBuildArgs& InArgs) = 0;
		virtual void PopulateSearchString(const TTreeItemType& InItem, TArray<FString>& InOutSearchStrings) const {}

		virtual bool CanBeSorted() const { return false; }
		virtual bool IsLessThan(const TTreeItemType& Left, const TTreeItemType& Right) const { return false; }

		virtual ~IReplicationTreeColumn() = default;
	};
}