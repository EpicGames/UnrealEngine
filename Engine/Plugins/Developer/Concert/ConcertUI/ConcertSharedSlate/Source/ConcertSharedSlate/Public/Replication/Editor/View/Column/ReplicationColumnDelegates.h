// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IReplicationTreeColumn.h"

namespace UE::ConcertSharedSlate
{
	template<typename TTreeItemType>
	struct TReplicationColumnDelegates
	{
		DECLARE_DELEGATE_RetVal(TSharedRef<IReplicationTreeColumn<TTreeItemType>>, FCreateColumn);
	};
}
