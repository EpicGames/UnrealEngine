// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ReplicationColumnDelegates.h"

#include "HAL/Platform.h"

namespace UE::ConcertSharedSlate
{
	/**
	 * Meta information about a column that can change independent of the IReplicationTreeColumn implementation.
	 * For example, IReplicationTreeColumn does not need to know about sort order.
	 */
	struct FReplicationColumnInfo
	{
		/** Determines whether this column is the first, etc. */
		int32 SortOrder = 0;
	};

	template<typename TTreeItemType>
	struct TReplicationColumnEntry
	{
		/** Factory function for creating the column instance. */
		typename TReplicationColumnDelegates<TTreeItemType>::FCreateColumn CreateColumn;
		/** The ID of the created column. Must be equal to what IReplicationTreeColumn::CreateHeaderRowArgs would return. */
		FName ColumnId;
		/** Additional info about the column that is not directly controlled by IReplicationTreeColumn. */
		FReplicationColumnInfo ColumnInfo;
	};
}
