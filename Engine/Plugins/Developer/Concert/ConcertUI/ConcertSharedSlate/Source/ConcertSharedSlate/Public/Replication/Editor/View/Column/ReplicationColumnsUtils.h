// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Editor/View/Column/IReplicationTreeColumn.h"
#include "Replication/Editor/View/Column/ReplicationColumnDelegates.h"
#include "Replication/Editor/View/Column/ReplicationColumnInfo.h"

#include "Containers/ArrayView.h"
#include "Widgets/Input/SCheckBox.h"
#include "Templates/SharedPointer.h"

namespace UE::ConcertSharedSlate
{
	class IEditableReplicationStreamModel;
	class IReplicationStreamViewer;
	class IReplicationStreamModel;
}

namespace UE::ConcertSharedSlate
{
	/** Info about a column that is being sorted. */
	struct FColumnSortInfo
	{
		FName SortedColumnId;
		EColumnSortMode::Type SortMode = EColumnSortMode::None;

		bool IsValid() const { return SortedColumnId != NAME_None && SortMode != EColumnSortMode::None; }
	};
	
	template<typename TListItemType>
	struct TCheckboxColumnDelegates
	{
		DECLARE_DELEGATE_RetVal_OneParam(ECheckBoxState, FGetColumnCheckboxState, const TListItemType& /*ObjectData*/);
		DECLARE_DELEGATE_TwoParams(FOnColumnCheckboxChanged, bool /*bIsChecked*/, const TListItemType& /*ObjectData*/);
		DECLARE_DELEGATE_RetVal_OneParam(bool, FIsEnabled, const TListItemType& /*ObjectData*/);
		DECLARE_DELEGATE_RetVal_OneParam(FText, FGetToolTipText, const TListItemType& /*ObjectData*/);

		TCheckboxColumnDelegates(
			FGetColumnCheckboxState InGetCheckboxStateDelegate,
			FOnColumnCheckboxChanged InOnCheckboxChangedDelegate,
			FGetToolTipText InGetToolTipTextDelegate = {},
			FIsEnabled InIsEnabledDelegate = {}
			)
			: GetCheckboxStateDelegate(MoveTemp(InGetCheckboxStateDelegate))
			, OnCheckboxChangedDelegate(MoveTemp(InOnCheckboxChangedDelegate))
			, GetToolTipTextDelegate(MoveTemp(InGetToolTipTextDelegate))
			, IsEnabledDelegate(MoveTemp(InIsEnabledDelegate))
		{}
		
		TCheckboxColumnDelegates(
			FGetColumnCheckboxState InGetCheckboxStateDelegate,
			FOnColumnCheckboxChanged InOnCheckboxChangedDelegate,
			TAttribute<FText> InToolTipTextAttribute,
			FIsEnabled InIsEnabledDelegate = {}
			)
			: GetCheckboxStateDelegate(MoveTemp(InGetCheckboxStateDelegate))
			, OnCheckboxChangedDelegate(MoveTemp(InOnCheckboxChangedDelegate))
			, GetToolTipTextDelegate(FGetToolTipText::CreateLambda([GetToolTipText = MoveTemp(InToolTipTextAttribute)](const TListItemType&){ return GetToolTipText.Get(); }))
			, IsEnabledDelegate(MoveTemp(InIsEnabledDelegate))
		{}

		FGetColumnCheckboxState GetCheckboxStateDelegate;
		FOnColumnCheckboxChanged OnCheckboxChangedDelegate;

		/** Optional. Defaults to FText::GetEmpty() */
		FGetToolTipText GetToolTipTextDelegate;
		/** Optional. Default to true. */
		FIsEnabled IsEnabledDelegate;
	};

	/** Util for making a checkbox column */
	template<typename TListItemType>
	UE::ConcertSharedSlate::template TReplicationColumnEntry<TListItemType> MakeCheckboxColumn(
		FName ColumnId,
		TCheckboxColumnDelegates<TListItemType> Delegates,
		FText DefaultLabel,
		const int32 Priority,
		const float ColumnWidth = 20.f
		)
	{
		class TCheckboxColumn : public UE::ConcertSharedSlate::template IReplicationTreeColumn<TListItemType>
		{
		public:

			TCheckboxColumn(
				FName ColumnId,
				FText DefaultLabel,
				const float ColumnWidth,
				TCheckboxColumnDelegates<TListItemType> Delegates
				)
				: ColumnId(ColumnId)
				, DefaultLabel(MoveTemp(DefaultLabel))
				, ColumnWidth(ColumnWidth)
				, Delegates(MoveTemp(Delegates))
			{}
			
			virtual SHeaderRow::FColumn::FArguments CreateHeaderRowArgs() const override
			{
				return SHeaderRow::Column(ColumnId)
					.DefaultLabel(DefaultLabel)
					.FixedWidth(ColumnWidth);
			}
			virtual TSharedRef<SWidget> GenerateColumnWidget(const typename IReplicationTreeColumn<TListItemType>::FBuildArgs& InArgs) override
			{
				return SNew(SCheckBox)
					.ToolTipText_Lambda([GetToolTipDelegate = Delegates.GetToolTipTextDelegate, RowData = InArgs.RowItem]()
					{
						return GetToolTipDelegate.IsBound() ? GetToolTipDelegate.Execute(RowData) : FText::GetEmpty();
					})
					.IsEnabled_Lambda([IsEnabledDelegate = Delegates.IsEnabledDelegate, RowData = InArgs.RowItem]()
					{
						return !IsEnabledDelegate.IsBound() || IsEnabledDelegate.Execute(RowData);
					})
					.IsChecked_Lambda([IsCheckedDelegate = Delegates.GetCheckboxStateDelegate, RowData = InArgs.RowItem]()
					{
						return IsCheckedDelegate.Execute(RowData);
					})
					.OnCheckStateChanged_Lambda([OnChangedDelegate = Delegates.OnCheckboxChangedDelegate, RowData = InArgs.RowItem](ECheckBoxState NewState)
					{
						const bool bIsChecked = NewState == ECheckBoxState::Checked;
						OnChangedDelegate.Execute(bIsChecked, RowData);
					});
			}
			
			virtual void PopulateSearchString(const TListItemType& InItem, TArray<FString>& InOutSearchStrings) const override {}
			
			virtual bool CanBeSorted() const override { return true; }
			virtual bool IsLessThan(const TListItemType& Left, const TListItemType& Right) const override
			{
				const bool bIsLeftChecked = Delegates.GetCheckboxStateDelegate.IsBound() && Delegates.GetCheckboxStateDelegate.Execute(Left) == ECheckBoxState::Checked;
				const bool bIsRightChecked = Delegates.GetCheckboxStateDelegate.IsBound() && Delegates.GetCheckboxStateDelegate.Execute(Right) == ECheckBoxState::Checked;
				// Less for checkbox means: checked < unchecked. !bIsRightChecked because checked == checked so it cannot be less.
				return bIsLeftChecked && !bIsRightChecked;
			}

		private:

			const FName ColumnId;
			const FText DefaultLabel;
			const float ColumnWidth;
			const TCheckboxColumnDelegates<TListItemType> Delegates;
		};
		
		check(Delegates.GetCheckboxStateDelegate.IsBound() && Delegates.OnCheckboxChangedDelegate.IsBound());
		return 
		{
			UE::ConcertSharedSlate::template TReplicationColumnDelegates<TListItemType>::FCreateColumn::CreateLambda([ColumnId, DefaultLabel, ColumnWidth, Delegates = MoveTemp(Delegates)]()
			{
				return MakeShared<TCheckboxColumn>(ColumnId, DefaultLabel, ColumnWidth, Delegates);
			}),
			ColumnId,
			{ Priority }
		};
	}
}
