// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IObjectTreeColumn.h"
#include "IPropertyTreeColumn.h"
#include "Replication/Utils/ReplicationWidgetDelegates.h"
#include "ReplicationColumnInfo.h"

#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/SharedPointer.h"

namespace UE::ConcertSharedSlate
{
	struct FObjectTreeRowContext;
	struct FPropertyTreeRowContext;
	class IEditableReplicationStreamModel;
	class IReplicationStreamViewer;
	class IReplicationStreamModel;
	class IObjectNameModel;
}

namespace UE::ConcertSharedSlate::ReplicationColumns::TopLevel
{
	CONCERTSHAREDSLATE_API extern const FName LabelColumnId;
	CONCERTSHAREDSLATE_API extern const FName TypeColumnId;
	CONCERTSHAREDSLATE_API extern const FName NumPropertiesColumnId;

	enum class ETopLevelColumnOrder : int32
	{
		/** Label of the object */
		Label = 20,
		/** Class of the object */
		Type = 30,
		/** Displays the number of properties assigned to the object */
		NumProperties = 40
	};

	enum class ENumPropertiesFlags : uint8
	{
		None,
		/** Also add up the number of properties that child objects have assigned. */
		IncludeSubobjectCounts
	};
	ENUM_CLASS_FLAGS(ENumPropertiesFlags);

	/**
	 * @param OptionalNameModel Used to look object name. Defaults to name displayed in object path if unset.
	 * @param GetObjectClassDelegate Used to display class icon. No icon is displayed if unset.
	 */
	CONCERTSHAREDSLATE_API FObjectColumnEntry LabelColumn(IObjectNameModel* OptionalNameModel = nullptr, FGetObjectClass GetObjectClassDelegate = {});
	/** Shows the object's type. */
	CONCERTSHAREDSLATE_API FObjectColumnEntry TypeColumn(FGetObjectClass GetObjectClassDelegate);
	/** Shows the number of widgets assigned to the object. */
	CONCERTSHAREDSLATE_API FObjectColumnEntry NumPropertiesColumn(const IReplicationStreamModel& Model UE_LIFETIMEBOUND, ENumPropertiesFlags Flags = ENumPropertiesFlags::IncludeSubobjectCounts);
}

namespace UE::ConcertSharedSlate::ReplicationColumns::Property
{
	CONCERTSHAREDSLATE_API extern const FName LabelColumnId;
	CONCERTSHAREDSLATE_API extern const FName TypeColumnId;
	
	enum class EReplicationPropertyColumnOrder : int32
	{
		/** Label of the property */
		Label = 10,
		/** Type of the property */
		Type = 20
	};
	
	CONCERTSHAREDSLATE_API FPropertyColumnEntry LabelColumn();
	CONCERTSHAREDSLATE_API FPropertyColumnEntry TypeColumn();
}
