// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "UObject/SoftObjectPtr.h"

class FString;
class FText;
class SWidget;

namespace UE::ConcertSharedSlate
{
	class FPropertyData;
	class IObjectNameModel;
	
	struct FCategoryRowGenerationArgs
	{
		/** The objects for which this is being generated */
		TArray<TSoftObjectPtr<>> ContextObjects;
		/** The highlight text which contains the search terms*/
		TAttribute<FText> HighlightText;

		FCategoryRowGenerationArgs(TArray<TSoftObjectPtr<>> ContextObjects, TAttribute<FText> HighlightText)
			: ContextObjects(MoveTemp(ContextObjects))
			, HighlightText(MoveTemp(HighlightText))
		{}
	};
	
	/** A special reassignment view which shows the display object and all its children. */
	class ICategoryRow
	{
	public:

		/** Generates search terms */
		virtual void GenerateSearchTerms(const TArray<TSoftObjectPtr<>>& ContextObjects, TArray<FString>& InOutSearchTerms) const = 0;

		/** Gets the row as widget */
		virtual TSharedRef<SWidget> GetWidget() = 0;

		virtual ~ICategoryRow() = default;
	};
	
	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<ICategoryRow>, FCreateCategoryRow, const FCategoryRowGenerationArgs& Args);
	
	/** Creates a delegate that generates a row displaying the name of the first context object. */
	CONCERTSHAREDSLATE_API FCreateCategoryRow CreateDefaultCategoryGenerator(TSharedRef<IObjectNameModel> NameModel);
}
