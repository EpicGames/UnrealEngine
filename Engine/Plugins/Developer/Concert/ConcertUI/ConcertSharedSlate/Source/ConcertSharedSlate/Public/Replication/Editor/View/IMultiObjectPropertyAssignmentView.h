// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyAssignmentView.h"

namespace UE::ConcertSharedSlate
{
	/** A special reassignment view which shows the display object and all its children. */
	class IMultiObjectPropertyAssignmentView : public IPropertyAssignmentView
	{
	public:

		/** Sets whether subobjects should be shown in the view (applies only to subobjects other than UActorComponents, which are always shown). */
		virtual void SetShouldShowSubobjects(bool bShowSubobjects) = 0;
		/** @return Whether subobjects should be shown */
		virtual bool GetShouldShowSubobjects() const = 0;
	};
}
