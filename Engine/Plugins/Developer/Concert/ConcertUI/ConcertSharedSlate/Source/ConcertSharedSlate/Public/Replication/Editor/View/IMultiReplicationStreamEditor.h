// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

namespace UE::ConcertSharedSlate
{
	class IEditableMultiReplicationStreamModel;
	class IEditableReplicationStreamModel;
	class IReplicationStreamEditor;
	class IReplicationStreamModel;
	
	/**
	 * Widget which edits multiple replication stream.
	 * @see ReplicationWidgetFactories.h
	 */
	class IMultiReplicationStreamEditor : public SCompoundWidget
	{
	public:

		/** @return The widget drawing the consolidated model. */
		virtual IReplicationStreamEditor& GetEditorBase() const = 0;

		/** @return The source of the sub-streams */
		virtual IEditableMultiReplicationStreamModel& GetMultiStreamModel() const = 0;

		/** @return Gets a model that combines all streams into one. */
		virtual IEditableReplicationStreamModel& GetConsolidatedModel() const = 0;
	};
}
