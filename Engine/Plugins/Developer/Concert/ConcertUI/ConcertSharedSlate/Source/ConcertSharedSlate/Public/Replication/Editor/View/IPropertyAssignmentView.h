// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/Set.h"
#include "Templates/SharedPointer.h"
#include "UObject/SoftObjectPtr.h"

class SWidget;
class UObject;

namespace UE::ConcertSharedSlate
{
	class IReplicationStreamModel;

	/**
	 * An object group is a bunch of related objects.
	 * This relates to multi-editing.
	 *
	 * Example: You click 2 ACineCameraActor in the IReplicationStreamViewer:
	 * - A group is the 2 actors you clicked
	 * - A group is the two cine camera components of the actor
	 */
	struct FObjectGroup
	{
		TArray<TSoftObjectPtr<>> Group;
	};
	
	/**
	 * A replication assignment view displays an object's properties.
	 * It is the piece of UI that is displayed on the bottom part of the replication viewer.
	 * 
	 * The replication stream viewer / editor looks like this (@see CreateBaseStreamEditor):
	 * - Object tree view: User can click on an object.
	 * - IPropertyAssignmentView: The clicked object's properties are displayed (the "root objects").
	 *
	 * Right now there are 2 implementation:
	 *	- SPerObjectPropertyAssignment, which displays the root object's properties only
	 *	- SMultiObjectAssignment, which displays the root object and its subobjects.
	 * Generally, you can imagine the view as a tree view which has columns that can be injected (e.g. @see CreateBaseStreamEditor).
	 */
	class IPropertyAssignmentView
	{
	public:

		/**
		 * Rebuilds all displayed data immediately.
		 *
		 * @param Objects The objects that are supposed to be displayed.
		 * @param Model The model that can be queried for object info.
		 */
		virtual void RefreshData(const TArray<TSoftObjectPtr<>>& Objects, const IReplicationStreamModel& Model) = 0;
		
		/** Reapply the filter function to all items at the end of the frame. Call e.g. when the filters have changed. */
		virtual void RequestRefilter() const = 0;
		
		/**
		 * Requests that the given column be resorted, if it currently affects the row sorting (primary or secondary).
		 * Call e.g. when a sortable attribute of the column has changed.
		 */
		virtual void RequestResortForColumn(const FName& ColumnId) = 0;

		/** Gets the tree view's widget */
		virtual TSharedRef<SWidget> GetWidget() = 0;

		/** @return The groups of objects being displayed in this view right now. */
		virtual TArray<FObjectGroup> GetDisplayedGroups() const = 0;
		
		DECLARE_MULTICAST_DELEGATE(FOnSelectionChanged);
		/** @return Event that broadcasts when the result of GetDisplayedGroups() has changed. */
		virtual FOnSelectionChanged& OnObjectGroupsChanged() = 0;

		virtual ~IPropertyAssignmentView() = default;
	};
}
