// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Data/ConcertPropertySelection.h"

#include "Containers/Set.h"
#include "Templates/SharedPointer.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"

class SWidget;

namespace UE::ConcertSharedSlate
{
	struct FPropertyAssignmentEntry
	{
		/**
		 * The objects for which the properties are being displayed.
		 *
		 * This usually has only 1 entry.
		 * This has multiple elements in the case of multi-edit (i.e. when the user clicks multiple, compatible actors in the top-view).
		 * For example, for multi-edit this could contain ActorA->StaticMeshComponent0 and ActorB->StaticMeshComponent0.
		 */
		TArray<TSoftObjectPtr<>> ContextObjects;
		
		/** The properties to display */
		TSet<FConcertPropertyChain> PropertiesToDisplay;
		/** The class of the properties */
		FSoftClassPath Class;
	};
	
	/** Represents a tree view displaying properties from a single class. */
	class IPropertyTreeView
	{
	public:
		
		/**
		 * Rebuilds all property data from the property source.
		 *
		 * @param PropertiesToDisplay The properties to display
		 * @param Class The class from which the PropertiesToDisplay come
		 * @param bCanReuseExistingRowItems True, will try to reuse rows for properties in the tree already (retains selected rows).
		 *	Set this to false, if all rows should be regenerated (clears selection).
		 *	In general, always set this to false if you've changed the object for which you're displaying the class.
		 */
		UE_DEPRECATED(5.5, "Use the version that takes FPropertyAssignmentEntry instead")
		void RefreshPropertyData(const TSet<FConcertPropertyChain>& PropertiesToDisplay, const FSoftClassPath& Class, bool bCanReuseExistingRowItems = true) {}

		/**
		 * Rebuilds all property data from the property source.
		 *
		 * @param Entries Defines the property content to display
		 * @param bCanReuseExistingRowItems True, will try to reuse rows for properties in the tree already (retains selected rows).
		 *	Set this to false, if all rows should be regenerated (clears selection).
		 *	In general, always set this to false if you've changed the object for which you're displaying the class.
		 */
		virtual void RefreshPropertyData(const TArray<FPropertyAssignmentEntry>& Entries, bool bCanReuseExistingRowItems = true) = 0;
		
		/** Reapply the filter function to all items at the end of the frame. Call e.g. when the filters have changed. */
		virtual void RequestRefilter() const = 0;
		
		/**
		 * Requests that the given column be resorted, if it currently affects the row sorting (primary or secondary).
		 * Call e.g. when a sortable attribute of the column has changed.
		 */
		virtual void RequestResortForColumn(const FName& ColumnId) = 0;

		/** Scroll the given property into view, if it is contained. */
		virtual void RequestScrollIntoView(const FConcertPropertyChain& PropertyChain) = 0;

		/** Gets the tree view's widget */
		virtual TSharedRef<SWidget> GetWidget() = 0;

		virtual ~IPropertyTreeView() = default;
	};
}
