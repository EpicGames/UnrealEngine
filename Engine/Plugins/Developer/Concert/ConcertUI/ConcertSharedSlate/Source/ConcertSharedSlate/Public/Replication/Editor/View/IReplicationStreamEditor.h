// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IReplicationStreamViewer.h"
#include "Templates/SharedPointer.h"

namespace UE::ConcertSharedSlate
{
	/**
	 * Widget which edits replication stream.
	 * @see ReplicationWidgetFactories.h
	 */
	class IReplicationStreamEditor : public IReplicationStreamViewer
	{};
}
