// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API CONCERTSHAREDSLATE_API

namespace UE::ConcertSharedSlate
{
	/**
	 * Widget which views a replication stream.
	 * @see ReplicationWidgetFactories.h
	 */
	class IReplicationStreamViewer : public SCompoundWidget
	{
	public:

		/** Call after the data underlying the model was externally changed and needs to be redisplayed in the UI. */
		virtual void Refresh() = 0;

		/**
		 * Requests that column be resorted; the column is in the top object view.
		 * This is to be called in response to a column's content changing. The rows will be resorted if the given column has a sort priority assigned.
		 */
		virtual void RequestObjectColumnResort(const FName& ColumnId) = 0;
		/**
		 * Requests that column be resorted; the column is in the bottom property view.
		 * This is to be called in response to a column's content changing. The rows will be resorted if the given column has a sort priority assigned.
		 */
		virtual void RequestPropertyColumnResort(const FName& ColumnId) = 0;

		/** @return The objects for which the properties are being edited / displayed. */
		UE_DEPRECATED(5.5, "Use GetSelectedObjects instead.")
		UE_API virtual TArray<FSoftObjectPath> GetObjectsBeingPropertyEdited() const;

		/** Selects the objects in the outliner and deselects all others. Only actors can be selected. Invalid entries are filtered out. */
		virtual void SetSelectedObjects(TConstArrayView<TSoftObjectPtr<>> Objects) = 0;
		/** @return The objects for which the properties are being edited / displayed. */
		virtual TArray<TSoftObjectPtr<>> GetSelectedObjects() const = 0;

		virtual ~IReplicationStreamViewer() = default;
	};
}

#undef UE_API
