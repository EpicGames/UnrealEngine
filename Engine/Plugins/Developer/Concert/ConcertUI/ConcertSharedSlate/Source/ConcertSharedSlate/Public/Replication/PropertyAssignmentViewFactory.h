// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyTreeFactory.h"
#include "Editor/View/Column/IPropertyTreeColumn.h"
#include "Replication/Utils/FilterResult.h"

#include "Delegates/Delegate.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"

namespace UE::ConcertSharedSlate
{
	class IObjectHierarchyModel;
	class IMultiObjectPropertyAssignmentView;
	class FPropertyData;
	class IPropertyAssignmentView;
	class IPropertySourceProcessor;
	class IPropertyTreeView;
	
	DECLARE_DELEGATE_RetVal_OneParam(EFilterResult, FFilterPropertyData, const FPropertyData&);
	
	struct FCreatePerObjectAssignmentViewParams
	{
		/** Required. Displays the properties in a tree view. You can pass in e.g. custom UI with advanced filtering. */
		TSharedRef<IPropertyTreeView> PropertyTreeView = CreateSearchablePropertyTreeView();

		/**
		 * Optional.
		 * If specified, the view will display all properties reported by the model (useful for editor UI which edits streams, not useful for server where property info is not available).
		 * If unspecified, only display the properties assigned to the object in the stream.
		 */
		TSharedPtr<IPropertySourceProcessor> PropertySource;
	};
	
	/**
	 * Creates a view that shows the properties of the object the user clicks.
	 * 
	 * This basically just wraps IPropertyTreeView.
	 * You can customize the tree view by injecting columns (@see CreateSearchablePropertyTreeView).
	 */
	CONCERTSHAREDSLATE_API TSharedRef<IPropertyAssignmentView> CreatePerObjectAssignmentView(FCreatePerObjectAssignmentViewParams Params = {});
	
	struct FCreateMultiObjectAssignmentViewParams : FCreatePerObjectAssignmentViewParams
	{
		/** Required. Displays the properties in a tree view. You can pass in e.g. custom UI with advanced filtering. */
		TSharedRef<IPropertyTreeView> PropertyTreeView = CreateSearchablePropertyTreeView();

		/**
		 * Optional.
		 * Gets components and subobjects of the displayed object.
		 * If this is unspecified, the created IMultiObjectPropertyAssignmentView will behave exactly as the per object view (IPropertyAssignmentView).
		 */
		TSharedPtr<IObjectHierarchyModel> ObjectHierarchy;
		
		/**
		 * Optional.
		 * If specified, the view will display all properties reported by the model (useful for editor UI which edits streams, not useful for server where property info is not available).
		 * If unspecified, only display the properties assigned to the object in the stream.
		 */
		TSharedPtr<IPropertySourceProcessor> PropertySource;
	};

	/**
	 * Creates a view that shows the properties of the object the user clicks and its subobjects.
	 * You can customize the tree view by injecting columns (@see CreateSearchablePropertyTreeView).
	 */
	CONCERTSHAREDSLATE_API TSharedRef<IMultiObjectPropertyAssignmentView> CreateMultiObjectAssignmentView(FCreateMultiObjectAssignmentViewParams Params = {});
}
