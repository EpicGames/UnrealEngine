// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editor/View/Column/IPropertyTreeColumn.h"
#include "Editor/View/Column/ReplicationColumnsUtils.h"
#include "Editor/View/Column/SelectionViewerColumns.h"
#include "Editor/View/Extension/CategoryRowGeneration.h"
#include "Replication/Utils/FilterResult.h"

#include "Containers/ContainersFwd.h"
#include "Delegates/Delegate.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "UObject/SoftObjectPtr.h"

class SExpanderArrow;

namespace UE::ConcertSharedSlate
{
	class FPropertyData;
	class IObjectNameModel;
	class IPropertyTreeView;
	
	DECLARE_DELEGATE_RetVal_OneParam(EFilterResult, FFilterPropertyData, const FPropertyData&);
	
	struct FCreatePropertyTreeViewParams
	{
		/** Optional. Additional property columns you want added. */
		TArray<FPropertyColumnEntry> PropertyColumns
		{
			ReplicationColumns::Property::LabelColumn()
		};

		/** Optional filter function. */
		FFilterPropertyData FilterItem;

		/**
		 * Optional delegate for grouping objects under a category.
		 * If unset, no category are generated.
		 * 
		 * When the user clicks an object in the top view, this delegate will be called for the clicked object, its components (if an actor), and its (nested) subobjects.
		 * ContextObjects is a single object if a single object is clicked or multiple object in the case of multi-edit.
		 */
		FCreateCategoryRow CreateCategoryRow;
		
		/** Optional initial primary sort mode for object rows */
		FColumnSortInfo PrimaryPropertySort { ReplicationColumns::Property::LabelColumnId, EColumnSortMode::Ascending };
		/** Optional initial secondary sort mode for object rows */
		FColumnSortInfo SecondaryPropertySort { ReplicationColumns::Property::LabelColumnId, EColumnSortMode::Ascending };
		
		/** Optional widget to add to the left of the property list search bar. */
		TAlwaysValidWidget LeftOfPropertySearchBar;
		/** Optional widget to add to the right of the property list search bar. */
		TAlwaysValidWidget RightOfPropertySearchBar;
		/** Optional widget to add between the search bar and the table view (e.g. a SBasicFilterBar). */
		TAlwaysValidWidget RowBelowSearchBar;
		/** Optional, alternate content to show instead of the tree view when there are no rows. */
		TAlwaysValidWidget NoItemsContent;
	};
	
	/**
	 * Creates a tree view that uses a search box for filtering items.
	 * You can customize this tree view by adding custom widgets and columns into the property view.
	 */
	CONCERTSHAREDSLATE_API TSharedRef<IPropertyTreeView> CreateSearchablePropertyTreeView(FCreatePropertyTreeViewParams Params = {});
}
