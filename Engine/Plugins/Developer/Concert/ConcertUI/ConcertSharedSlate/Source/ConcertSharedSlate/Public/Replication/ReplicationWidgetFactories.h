// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyAssignmentViewFactory.h"
#include "Editor/View/Column/IObjectTreeColumn.h"
#include "Editor/View/Column/ReplicationColumnInfo.h"
#include "Editor/View/Column/SelectionViewerColumns.h"
#include "Replication/Editor/View/Column/ReplicationColumnsUtils.h"
#include "Replication/Utils/ReplicationWidgetDelegates.h"

#include "Delegates/Delegate.h"
#include "Misc/Attribute.h"
#include "Types/SlateEnums.h"
#include "Templates/SharedPointer.h"

class UObject;

struct FConcertStreamObjectAutoBindingRules;
struct FConcertObjectReplicationMap;

namespace UE::ConcertSharedSlate
{
	class IEditableMultiReplicationStreamModel;
	class IEditableReplicationStreamModel;
	class IMultiReplicationStreamEditor;
	class IObjectNameModel;
	class IObjectSelectionSourceModel;
	class IPropertyAssignmentView;
	class IPropertyTreeView;
	class IReplicationStreamModel;
	class IReplicationStreamEditor;
	class IReplicationStreamViewer;
	class IStreamExtender;
	class IObjectHierarchyModel;
	class IPropertySourceProcessor;

	/** Creates a stream model that will only read from ReplicationMapAttribute. */
	CONCERTSHAREDSLATE_API TSharedRef<IReplicationStreamModel> CreateReadOnlyStreamModel(
		TAttribute<const FConcertObjectReplicationMap*> ReplicationMapAttribute
		);
	
	/**
	 * Creates a model that can be passed to CreateEditor.
	 * 
	 * @param ReplicationMapAttribute Getter for extracting the FConcertObjectReplicationMap to edit
	 * @param Extender Optional callbacks for adding additional properties and objects when an object is added to the model
	 * 
	 * @return A model that will edit the FConcertObjectReplicationMap.
	 */
	CONCERTSHAREDSLATE_API TSharedRef<IEditableReplicationStreamModel> CreateBaseStreamModel(
		TAttribute<FConcertObjectReplicationMap*> ReplicationMapAttribute,
		TSharedPtr<IStreamExtender> Extender = nullptr
		);
	
	/** Params for creating a IReplicationStreamViewer. */
	struct FCreateViewerParams
	{
		/** Required. In the lower half of the editor, this view presents the properties associated with the object that is currently selected in the upper part of the view. */
		TSharedRef<IPropertyAssignmentView> PropertyAssignmentView = CreatePerObjectAssignmentView();
		
		/**
		 * Optional. Determines the objects displayed as children to the top-level objects in the top section.
		 * If left unspecified, the top will only display actors.
		 * @note The created view will keep a strong reference to this.
		 */
		TSharedPtr<IObjectHierarchyModel> ObjectHierarchy;

		/** Optional. Determines the display name of objects. */
		TSharedPtr<IObjectNameModel> NameModel;

		/** Called to generate the context menu for objects. This extends the options already generated by this widget and is called at the end. */
		FExtendObjectMenu OnExtendObjectsContextMenu;
		
		/** Optional. Additional object columns you want added. */
		TArray<FObjectColumnEntry> ObjectColumns;
		
		/** Optional initial primary sort mode for object rows */
		FColumnSortInfo PrimaryObjectSort;
		/** Optional initial secondary sort mode for object rows */
		FColumnSortInfo SecondaryObjectSort;
		
		/** Optional widget to add to the left of the object list search bar. */
		TAlwaysValidWidget LeftOfObjectSearchBar;
		/** Optional widget to add to the right of the object list search bar. */
		TAlwaysValidWidget RightOfObjectSearchBar;
		
		/**
		 * Optional. Whether a given object should be displayed. 
		 *
		 * This is useful e.g. for hiding objects that are not in the local editor's opened world (for that purpuse couple this delegate with @see FHideObjectsNotInWorldLogic).
		 * If the list of displayed objects changes, call IReplicationStreamViewer::Refresh.
		 * 
		 * If this returns false on an object, none of its children will be shown either
		 * Child objects are determined using ObjectHierarchy.
		 */
		FShouldDisplayObject ShouldDisplayObjectDelegate;

		/** Optional. Delegate executed to create a widget that overlays an object row. */
		FMakeObjectRowOverlayWidget MakeObjectRowOverlayWidgetDelegate;
		/** Optional. If MakeRowOverlayWidgetDelegate is specified, this controls how the widget is aligned in the column. */
		EHorizontalAlignment OverlayWidgetAlignment = HAlign_Right;

		/** Optional. Returns a new widget that wraps the outliner widget, which displays the replicated objects. Could return e.g. an SOverlay or SDropTarget. */
		FWrapOutlinerWidget WrapOutlinerWidgetDelegate;
	};

	/** Params for creating an IReplicationStreamEditor */
	struct FCreateEditorParams
	{
		/**
		 * Required. The model that the editor is displaying.
		 * @note The view will keep a strong reference to this.
		 */
		TSharedRef<IEditableReplicationStreamModel> DataModel;
		
		/**
		 * Required. Determines the objects that can be added to the object list. 
		 * @note The view will keep a strong reference to this.
		 */
		TSharedRef<IObjectSelectionSourceModel> ObjectSource;
		/**
		 * Required. Determines the properties that are displayed in the property list. 
		 * @note The view will keep a strong reference to this.
		 */
		TSharedRef<IPropertySourceProcessor> PropertySource;
		
		/** Optional. Determines whether all UI for changing the model should be disabled. */
		TAttribute<bool> IsEditingEnabled;
		/** Optional. Whenever IsEditingEnabled returns true, this tooltip is displayed for relevant, disabled UI. */
		TAttribute<FText> EditingDisabledToolTipText;

		/** Called just before the passed objects are added to the stream model. Called in response to the user selecting these objects from the combo button. */
		FSelectObjectsFromComboButton OnPreAddSelectedObjectsDelegate;
		/** Called just after the passed objects have been added to the stream model. Called in response to the user selecting these objects from the combo button. */
		FSelectObjectsFromComboButton OnPostAddSelectedObjectsDelegate;
	};

	/**
	 * Creates a base replication stream editor.
	 * 
	 * This editor implements base functionality for adding objects but has NO logic for editing properties.
	 * You are expected to inject widgets or columns, which implement editing properties.
	 * @see ConcertClientSharedSlate::CreateDefaultStreamEditor, which adds a checkbox to the start of every property row.
	 *
	 * The editor consists of two or three areas (depending whether on whether SubobjectView was specified), which share similar workflows the level editor:
	 * 1. Root objects, similar to world outliner:
	 *		- Similar to World Outliner: Displays top-level objects, i.e. actors, are added here.
	 *		- FCreateEditorParams::ObjectSource is used to build a combo button through which new objects can be added.
	 * 2. Properties: Shows properties of the selected root object and / or subobjects.
	 */
	CONCERTSHAREDSLATE_API TSharedRef<IReplicationStreamEditor> CreateBaseStreamEditor(FCreateEditorParams EditorParams, FCreateViewerParams ViewerParams);

	DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<IEditableReplicationStreamModel>, FGetAutoAssignTarget, TConstArrayView<UObject*>);
	/** Params for creating an IMultiReplicationStreamEditor */
	struct FCreateMultiStreamEditorParams
	{
		/**
		 * The source of underlying streams that should be edited.
		 * @note The view will keep a strong reference to this.
		 */
		TSharedRef<IEditableMultiReplicationStreamModel> MultiStreamModel;

		/**
		 * This model consolidates all objects in all streams.
		 * 
		 * When an object is added (i.e. IEditableReplicationStreamModel::OnObjectsChanged broadcasts), e.g. when
		 * - to an object is added to one of the managed streams (e.g. stream is remotely changed),
		 * - via the "Add" button,
		 * it is added here, too. 
		 * This is needed for the internal operation of the multi view.
		 *
		 * You can make this model internally transact (@see ConcertClientSharedSlate::CreateTransactionalStreamModel).
		 * Doing so causes the objects added via "add" button to be transacted meaning the user can undo adding objects.
		 * 
		 * @note The view will keep a strong reference to this
		 */
		TSharedRef<IEditableReplicationStreamModel> ConsolidatedObjectModel;
		
		/**
		 * Determines the objects that are displayed in the property list. 
		 * @note The view will keep a strong reference to this.
		 */
		TSharedRef<IObjectSelectionSourceModel> ObjectSource;
		/**
		 * Determines the properties that can be added to the property list.
		 * @note The view will keep a strong reference to this.
		 */
		TSharedRef<IPropertySourceProcessor> PropertySource;

		/** Optional. If set, the Add button should automatically assign the added object to stream returned by this callback. */
		FGetAutoAssignTarget GetAutoAssignToStreamDelegate;
		
		/** Called just before the passed objects are added to the stream model. Called in response to the user selecting these objects from the combo button. */
		FSelectObjectsFromComboButton OnPreAddSelectedObjectsDelegate;
		/** Called just after the passed objects have been added to the stream model. Called in response to the user selecting these objects from the combo button. */
		FSelectObjectsFromComboButton OnPostAddSelectedObjectsDelegate;
	};

	/**
	 * Creates an editor that displays multiple streams in a single widget.
	 * 
	 * This editor consolidates all replicated objects into a single UI and display all properties in the class.
	 * You are supposed to inject custom columns into the property section for assigning specific properties
	 * to the streams in the IEditableReplicationStreamModel.
	 *
	 * @note This editor does NOT create any UI for changing the assigned properties. You are supposed to inject column widgets to the property rows for this.
	 */
	CONCERTSHAREDSLATE_API TSharedRef<IMultiReplicationStreamEditor> CreateBaseMultiStreamEditor(FCreateMultiStreamEditorParams EditorParams, FCreateViewerParams ViewerParams);
}
