// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::ConcertSharedSlate
{
	enum class EFilterResult : uint8
	{
		PassesFilter,
		DoesNotPassFilter
	};
}
