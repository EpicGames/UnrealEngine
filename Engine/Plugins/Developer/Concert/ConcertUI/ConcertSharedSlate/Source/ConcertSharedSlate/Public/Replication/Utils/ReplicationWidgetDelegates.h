// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Templates/SharedPointer.h"
#include "UObject/SoftObjectPtr.h"

class FMenuBuilder;
class SWidget;
enum class EBreakBehavior : uint8;
struct FConcertPropertyChain;
struct FReplicatedObjectData;
struct FSoftClassPath;
struct FSoftObjectPath;

namespace UE::ConcertSharedSlate
{
	class FPropertyData;
	class IPropertySourceProcessor;
	class IReplicationStreamModel;

	struct FSelectableObjectInfo;

	/** A predicate for determining Left < Right. */
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FSortPropertyPredicate, const FPropertyData& Left, const FPropertyData& Right);

	/** Extends a context menu that is being built for a selection of objects. */
	DECLARE_DELEGATE_TwoParams(FExtendObjectMenu, FMenuBuilder&, TConstArrayView<TSoftObjectPtr<>> ContextObjects);

	/** Delegate for getting an object's class. */
	DECLARE_DELEGATE_RetVal_OneParam(FSoftClassPath, FGetObjectClass, const TSoftObjectPtr<>&);

	/** Delegate for deciding whether an object should be displayed. */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FShouldDisplayObject, const FSoftObjectPath& ObjectPath);

	/** Delegate executed when object options are selected from the Add button to the left of the search bar in the top view of the replication panel. */
	DECLARE_DELEGATE_OneParam(FSelectObjectsFromComboButton, TConstArrayView<FSelectableObjectInfo>);

	/** Delegate executed to create a widget that overlays an object row. */
	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FMakeObjectRowOverlayWidget, const FReplicatedObjectData& ObjectData);

	/**
	 * Delegate executed during construction of UI. It receives the SReplicationTreeView showing the replicated objects.
	 * The returned widget is intended to wrap the tree in some way, e.g. by it into an SOverlay or SDropTarget.
	 */
	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FWrapOutlinerWidget, const TSharedRef<SWidget>& OutlinerWidget);
}