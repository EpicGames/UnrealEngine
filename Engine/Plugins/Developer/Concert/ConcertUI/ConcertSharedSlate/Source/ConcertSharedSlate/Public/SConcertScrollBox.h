// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/SlateDelegates.h"

#define UE_API CONCERTSHAREDSLATE_API

class SScrollBox;
class SButton;
class SScrollBar;


class SConcertScrollBox : public SCompoundWidget
{
public:

	class FSlot : public TSlotBase<FSlot>
	{
	public:
		using TSlotBase<FSlot>::TSlotBase;

		SLATE_SLOT_BEGIN_ARGS(FSlot, TSlotBase<FSlot>)
		SLATE_SLOT_END_ARGS()
	};

	SLATE_BEGIN_ARGS(SConcertScrollBox) {}
		SLATE_SLOT_ARGUMENT(FSlot, Slots)
	SLATE_END_ARGS();

	/**
	* Construct a custom ScrollBox that automatically scrolls to the bottom if the user is not currently scrolling.
	*
	* @param InArgs The Slate argument list.
	* @param ConstructUnderMajorTab The major tab which will contain the Session History widget.
	*/
	UE_API void Construct(const FArguments& InArgs);

	/** SWidget interface. */
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	/** End of SWidget interface. */

	/**
	 * @return a new slot. Slots contain children for SScrollBox 
	 */
	static UE_API FSlot::FSlotArguments Slot();

private:
	
	/** Delegate called to handle a user scrolling. */
	UE_API void HandleUserScrolled(float);

	/** Checks if the scrollbar is currently at the bottom of its track. */
	UE_API bool IsAtBottom();

	/** Callback for scrolling to the start of the list. */
	UE_API FReply HandleScrollToStart();

	/** Callback for scrolling to the end of the list. */
	UE_API FReply HandleScrollToEnd();

	/** Callback for getting the ScrollBar buttons visibility. */
	UE_API EVisibility HandleScrollButtonsVisibility() const;

	/** Creates a button with the FontAwesome font. */
	UE_API TSharedRef<SButton> CreateScrollBarButton(const FText& InToolTip, const FString& InIcon, FOnClicked OnClickedDelegate) const;

private:
	
	/** Whether the scrollbar is locked to the bottom. */
	bool bIsLocked;

	/** Whether locking should be prevented. */
	bool bPreventLock;

	/** Holds the ScrollBar. */
	TSharedPtr<SScrollBar> ScrollBar;

	/** Holds the inner ScrollBox. */
	TSharedPtr<SScrollBox> ScrollBox;
};

#undef UE_API
