// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Session/Activity/SConcertSessionActivities.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API CONCERTSHAREDSLATE_API

struct FConcertSessionActivity;
struct FConcertClientInfo;
class FConcertSessionActivitiesOptions;

/**
 * Displays the list of activities available for recovery and let the user select what should or shouldn't be recovered.
 */
class SConcertSessionRecovery : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SConcertSessionRecovery)
		: _WithClientAvatarColorColumn(false)
		, _WithClientNameColumn(false)
		, _WithOperationColumn(true)
		, _WithPackageColumn(false)
		, _DetailsAreaVisibility(EVisibility::Collapsed)
		, _IsConnectionActivityFilteringEnabled(false)
		, _IsLockActivityFilteringEnabled(false)
		, _IsPackageActivityFilteringEnabled(true)
		, _IsTransactionActivityFilteringEnabled(true)
		, _IsReplicationActivityFilteringEnabled(true)
		, _IsIgnoredActivityFilteringEnabled(false)
		, _AreRecoverAllAndCancelButtonsVisible(true)
		, _IsRecoverThroughButtonsVisible(true) {}

		/** An introduction text to put the user in context. */
		SLATE_ARGUMENT(FText, IntroductionText)

		/** The windows hosting this widget. */
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)

		/** If bound, invoked iteratively to populate the activity list. */
		SLATE_EVENT(SConcertSessionActivities::FFetchActivitiesFunc, OnFetchActivities)

		/** If bound, invoked to map an activity to a client info. */
		SLATE_EVENT(SConcertSessionActivities::FGetActivityClientInfoFunc, OnMapActivityToClient)

		/** Invoked when the user clicks the 'recover' button. */
		SLATE_ARGUMENT(TFunction<bool(TSharedPtr<FConcertSessionActivity>)>, OnRestore)

		/** Invoked when the user clicks the 'cancel' button. */
		SLATE_ARGUMENT(TFunction<void()>, OnCancel)

		/** Show/hide the column displaying the avatar color of the client who performed the activity. */
		SLATE_ARGUMENT(bool, WithClientAvatarColorColumn)

		/** Show/hide the column showing the display name of the client who performed the activity. */
		SLATE_ARGUMENT(bool, WithClientNameColumn)

		/** Show/hide the column displaying the operation represented by the activity. */
		SLATE_ARGUMENT(bool, WithOperationColumn)

		/** Show/hide the column displaying affected package. */
		SLATE_ARGUMENT(bool, WithPackageColumn)

		/** Show/hide the details area widget. */
		SLATE_ARGUMENT(EVisibility, DetailsAreaVisibility)

		/** Show/hide the check box in the 'View Options' to filter connection activities (join/leave session). */
		SLATE_ARGUMENT(bool, IsConnectionActivityFilteringEnabled)

		/** Show/hide the check box in the 'View Options' to filter lock activities (lock/unlock assets). */
		SLATE_ARGUMENT(bool, IsLockActivityFilteringEnabled)

		/** Show/hide the check box in the 'View Options' to filter package activities (create/delete/save/rename). */
		SLATE_ARGUMENT(bool, IsPackageActivityFilteringEnabled)

		/** Show/hide the check box in the 'View Options' to filter transaction activities. */
		SLATE_ARGUMENT(bool, IsTransactionActivityFilteringEnabled)
		
		/** Show/hide the check box in the 'View Options' to filter replication activities. */
		SLATE_ARGUMENT(bool, IsReplicationActivityFilteringEnabled)

		/** Show/hide the check box in the 'View Options' to filter unrecoverable activities. */
		SLATE_ARGUMENT(bool, IsIgnoredActivityFilteringEnabled)

		/** Show/hide the recover all and cancel buttons (so that if this widget is embedded in another one, the buttons can be replaced and put in a more appropriate location). */
		SLATE_ATTRIBUTE(bool, AreRecoverAllAndCancelButtonsVisible)

		/** Show/hide the recover though button. */
		SLATE_ATTRIBUTE(bool, IsRecoverThroughButtonsVisible)

		/** Show/hide a message overlay above the activities list explaining why no activities are displayed. */
		SLATE_ATTRIBUTE(FText, NoActivitiesReasonText)
	SLATE_END_ARGS();

	/**
	 * Construct the recovery widget.
	 * @param InArgs The widgets arguments and attributes.
	 */
	UE_API void Construct(const FArguments& InArgs);

	/** Returns the activity, selected by the user, through which the session should be (or was) recovered or null to prevent recovery. */
	TSharedPtr<FConcertSessionActivity> GetRecoverThroughItem() { return RecoveryThroughItem; }

	/** Removes all activities from the view and reset the activity stream. */
	UE_API void Reset();

	/** Returns the total number of activities currently stored (no filter applied). */
	UE_API int32 GetTotalActivityNum() const;

	/** Returns the tooltip shown for the 'Recovery All' button. */
	UE_API FText GetRecoverAllButtonTooltip() const;

	/** Returns the most recent activity available, ignoring the current filter. */
	UE_API TSharedPtr<FConcertSessionActivity> GetMostRecentActivity() const;

private:

	UE_API TArray<FActivityColumn> MakeColumns(const FArguments& InArgs) const;
	
	UE_API void OnSearchTextChanged(const FText& InFilterText);
	UE_API void OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type CommitType);
	UE_API FText HighlightSearchText() const;

	UE_API FReply OnCancelRecoveryClicked();
	UE_API FReply OnRecoverAllClicked();

	UE_API EVisibility GetRecoverThroughButtonVisibility(TSharedPtr<FConcertSessionActivity> Activity);
	UE_API FText GetRecoverThroughButtonTooltip() const;
	UE_API TSharedPtr<SWidget> MakeRecoverThroughWidget(TWeakPtr<FConcertSessionActivity>, const FName&);
	UE_API void RecoverThrough(TSharedPtr<FConcertSessionActivity> Item);

	/** Close the windows hosting this recovery widget. */
	UE_API void DismissWindow();

private:
	/** Display the session activities. */
	TSharedPtr<SConcertSessionActivities> ActivityView;

	/** Controls the various display options of the view. */
	TSharedPtr<FConcertSessionActivitiesOptions> ActivityViewOptions;

	/** The activity selected when the user click 'Recover' or 'Recover Through' buttons. */
	TSharedPtr<FConcertSessionActivity> RecoveryThroughItem;

	/** The parent window hosting this widget. */
	TWeakPtr<SWindow> ParentWindow;

	/** The widget used to enter the text to search. */
	TSharedPtr<SSearchBox> SearchBox;

	/** The search text entered in the search box. */
	FText SearchText;

	/** The text displayed at the top to summarize the purpose of the window. */
	FText IntroductionText;

	/** Externally control the 'recovery through' button visibility. The internal logic to show it is run first, if it is still visible, this is evaluated.*/
	TAttribute<bool> IsRecoverThroughButtonVisible;

	/** The function invoked when the user clicks the restore button. Might not be bound. */
	TFunction<bool(TSharedPtr<FConcertSessionActivity>)> OnRestoreFn;

	/** The function invoked when the user clicks the cancel button. Might not be bound. */
	TFunction<void()> OnCancelFn;
};

#undef UE_API
