// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

#define UE_API CONCERTSHAREDSLATE_API

struct FConcertPackageInfo;

class SPackageDetails : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPackageDetails)
	{}
	SLATE_END_ARGS()

	/**
	* Constructs a package details widget.
	*
	* @param InArgs The Slate argument list.
	*/
	UE_API void Construct(const FArguments& InArgs);

	/**
	 * Set the current package to have its details displayed.
	 * @param InPackageInfo The current package to have its info displayed.
	 * @param InRevision The target package's revision.
	 * @param InModifiedBy The username of the user who modified the package.
	 */
	UE_API void SetPackageInfo(const FConcertPackageInfo& InPackageInfo, const int64 InRevision, const FString& InModifiedBy);

private:

	/** Holds basic information about a given package. */
	struct FPackageDetailsRow
	{
		FText PackageName;
		FText Revision;
		FText ModifiedBy;
		FText Action;
	};

	using FListItemPtr = TSharedPtr<FPackageDetailsRow>;

	/** Callback for generating package info rows. */
	UE_API TSharedRef<ITableRow> HandleGenerateRow(FListItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable) const;

private:

	/** Holds the ListView of package details rows. */
	TSharedPtr<SListView<FListItemPtr>>	DetailListView;

	/** Holds the package details rows. */
	TArray<FListItemPtr> DetailItems;
};

#undef UE_API
