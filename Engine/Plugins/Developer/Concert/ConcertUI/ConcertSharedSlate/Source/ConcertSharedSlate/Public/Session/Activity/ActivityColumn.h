// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Views/SHeaderRow.h"

#define UE_API CONCERTSHAREDSLATE_API

class SConcertSessionActivities;
struct FConcertSessionActivity;

/** A column in SConcertSessionActivities */
class FActivityColumn final : public SHeaderRow::FColumn
{
public:

	/** Generates the widget content for a column in a row */
	DECLARE_DELEGATE_ThreeParams(FGenerateColumnWidget,
		const TSharedRef<SConcertSessionActivities>& Owner,
		const TSharedRef<FConcertSessionActivity>&,
		SOverlay::FScopedWidgetSlotArguments& Slot
		)

	/** Appends search terms to ExistingSearchStrings. */
	DECLARE_DELEGATE_ThreeParams(FPopulateSearchString,
		const TSharedRef<SConcertSessionActivities>& Owner,
		const FConcertSessionActivity& Activity,
		TArray<FString>& ExistingSearchStrings
		)
	
	FActivityColumn(const FArguments& InArgs)
		: FColumn(InArgs)
	{}

	/** Called to generate the column's widget for a row */
	FActivityColumn& GenerateColumnWidget(FGenerateColumnWidget Callback) { GenerateColumnWidgetCallback = MoveTemp(Callback); return *this; }
	/** Called to populate the search terms when rows are search */
	FActivityColumn& PopulateSearchString(FPopulateSearchString Callback) { PopulateSearchStringCallback = MoveTemp(Callback); return *this; }

	/** Determines whether this column is the first, etc. */
	FActivityColumn& ColumnSortOrder(int32 NewValue) { ColumnSortOrderValue = NewValue; return *this; }
	
	UE_API void BuildColumnWidget(const TSharedRef<SConcertSessionActivities>& Owner, const TSharedRef<FConcertSessionActivity>& Activity, SOverlay::FScopedWidgetSlotArguments& Slot) const;
	UE_API void ExecutePopulateSearchString(const TSharedRef<SConcertSessionActivities>& Owner, const FConcertSessionActivity& Activity, TArray<FString>& ExistingSearchStrings) const;
	int32 GetColumnSortOrderValue() const { return ColumnSortOrderValue; }
	
private:

	FGenerateColumnWidget GenerateColumnWidgetCallback;
	FPopulateSearchString PopulateSearchStringCallback;

	/** Determines whether this column is the first, etc. */
	int32 ColumnSortOrderValue = 0;
};

#undef UE_API
