// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/IReflectionDataProvider.h"

struct FConcertTransactionEventBase;

namespace UE::ConcertSharedSlate
{
	/** Passed on to SUndoHistoryDetails to display reflection data in non-editor builds. Used e.g. by the server. */
	class IConcertReflectionDataProvider : public UndoHistory::IReflectionDataProvider 
	{
	public:

		/** Called just before SUndoHistoryDetails::SetSelectedTransaction. Injects the related property data saved in the activity. */
		virtual void SetTransactionContext(const FConcertTransactionEventBase& InTransaction) = 0;
	};
}
