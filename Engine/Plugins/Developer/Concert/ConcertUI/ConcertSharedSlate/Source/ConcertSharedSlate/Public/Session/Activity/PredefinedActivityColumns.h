// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SConcertSessionActivities.h"

class FActivityColumn;

/** Contains factories for commonly used columns. By default, they maintain the relative order specified by EPredefinedColumnOrder. */
namespace UE::ConcertSharedSlate::ActivityColumn
{
	enum class EPredefinedColumnOrder : int32
	{
		AvatarColor = -20,
		DateTime = -10,
		ClientName = 10,
		Operation = 20,
		Package = 30
		// No Summary entry because it is forced to be last
	};
	
	extern const FName DateTimeColumnId;
	extern const FName OperationColumnId;
	extern const FName AvatarColorColumnId;
	extern const FName ClientNameColumnId;
	extern const FName PackageColumnId;
	extern const FName SummaryColumnId;
	
	// Required
	FActivityColumn DateTime();
	FActivityColumn Summary();

	// Optional
	CONCERTSHAREDSLATE_API FActivityColumn AvatarColor();
	CONCERTSHAREDSLATE_API FActivityColumn ClientName();
	CONCERTSHAREDSLATE_API FActivityColumn Operation();
	CONCERTSHAREDSLATE_API FActivityColumn Package();
}
