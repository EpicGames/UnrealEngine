// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ActivityColumn.h"
#include "ConcertFrontendUtils.h"

#include "Async/Future.h"
#include "ConcertHeaderRowUtils.h"
#include "ConcertSyncSessionTypes.h"
#include "Misc/TextFilter.h"
#include "Replication/Messages/ReplicationActivity.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Layout/SExpandableArea.h"

#define UE_API CONCERTSHAREDSLATE_API

struct FConcertSessionActivity;
struct FConcertClientInfo;
struct FConcertPackageInfo;
struct FConcertTransactionEventBase;
class SExpendableArea;
class SHeaderRow;
class SPackageDetails;
class SUndoHistoryDetails;
class STextBlock;

namespace UE::ConcertSharedSlate
{
	class IConcertReflectionDataProvider;
}

/** Filters for the concert session activity view. */
enum class EConcertActivityFilterFlags
{
	ShowAll                  = 0x00,
	HideConnectionActivities = 1<<0,
	HideLockActivities       = 1<<1,
	HidePackageActivities    = 1<<2,
	HideTransactionActivities= 1<<3,
	HideIgnoredActivities    = 1<<4,
	HideReplicationActivities= 1<<5
};
ENUM_CLASS_FLAGS(EConcertActivityFilterFlags);

/** Displays session activities in a table view. */
class SConcertSessionActivities : public SCompoundWidget
{
public:
	
	/** Used to pull activities from a session. Used to fetch and display the activities of an archived session. */
	DECLARE_DELEGATE_RetVal_ThreeParams(bool, FFetchActivitiesFunc,
		TArray<TSharedPtr<FConcertSessionActivity>>& /*InOutActivities*/,
		int32& /*OutFetchedCount*/,
		FText& /*ErrorMsg*/
		)
	
	/** Used to map an activity to its client. */
	DECLARE_DELEGATE_RetVal_OneParam(TOptional<FConcertClientInfo>, FGetActivityClientInfoFunc,
		FGuid /*ClientId*/
		)

	/** Returns the transaction event corresponding the specified activity.*/
	DECLARE_DELEGATE_RetVal_OneParam(TFuture<TOptional<FConcertSyncTransactionEvent>>, FGetTransactionEvent,
		const FConcertSessionActivity& /*Activity*/
		)
	
	/** Returns the package event corresponding to the package activity. */
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FGetPackageEvent,
		const FConcertSessionActivity& /*Activity*/,
		FConcertSyncPackageEventMetaData& /*OutEvent*/
		)

	/** Used to overlay a widget over a column widget to add custom functionalities to a row. */
	DECLARE_DELEGATE_RetVal_ThreeParams(TSharedPtr<SWidget>, FMakeColumnOverlayWidgetFunc,
		TWeakPtr<SMultiColumnTableRow<TSharedPtr<FConcertSessionActivity>>> /*OwningRow*/,
		TWeakPtr<FConcertSessionActivity> /*ThisRowActivity*/,
		const FName& /*ColumnId*/
		)

public:
	SLATE_BEGIN_ARGS(SConcertSessionActivities)
		: _OnFetchActivities()
		, _OnMapActivityToClient()
		, _OnMakeColumnOverlayWidget()
		, _TimeFormat(ETimeFormat::Relative)
		, _ConnectionActivitiesVisibility(EVisibility::Hidden)
		, _LockActivitiesVisibility(EVisibility::Hidden)
		, _PackageActivitiesVisibility(EVisibility::Visible)
		, _TransactionActivitiesVisibility(EVisibility::Visible)
		, _ReplicationActivitiesVisibility(EVisibility::Hidden)
		, _IgnoredActivitiesVisibility(EVisibility::Hidden)
		, _DetailsAreaVisibility(EVisibility::Hidden)
		, _IsAutoScrollEnabled(false)
		, _DarkenMutedActivities(true)
		{ }

		/** If bound, invoked to populate the view. */
		SLATE_EVENT(FFetchActivitiesFunc, OnFetchActivities)

		/** If bound, invoked to map an activity to a client.*/
		SLATE_EVENT(FGetActivityClientInfoFunc, OnMapActivityToClient)

		/** If bound, invoked to fill up the package activity details panel. */
		SLATE_EVENT(FGetPackageEvent, OnGetPackageEvent)

		/** If bound, invoked to fill up the transaction activity details panel. */
		SLATE_EVENT(FGetTransactionEvent, OnGetTransactionEvent)

		/** If bound, invoked when generating a row to add an overlay to a column. */
		SLATE_EVENT(FMakeColumnOverlayWidgetFunc, OnMakeColumnOverlayWidget)

		/** If bound, invoked when an item in the table is right-click and we are supposed to show a menu of actions. */
		SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)
	
		/** The columns to show for each activity; they are shown after date time */
		SLATE_ARGUMENT(TArray<FActivityColumn>, Columns)

		/** Highlight the returned text in the view. */
		SLATE_ATTRIBUTE(FText, HighlightText)

		/** Defines how time should be displayed (relative vs absolute). */
		SLATE_ATTRIBUTE(ETimeFormat, TimeFormat)

		/** Show/hide connection activities. */
		SLATE_ATTRIBUTE(EVisibility, ConnectionActivitiesVisibility)

		/** Show/hide lock activities. */
		SLATE_ATTRIBUTE(EVisibility, LockActivitiesVisibility)

		/** Show/hide package activities. */
		SLATE_ATTRIBUTE(EVisibility, PackageActivitiesVisibility)

		/** Show/hide transaction activities. */
		SLATE_ATTRIBUTE(EVisibility, TransactionActivitiesVisibility)
		
		/** Show/hide replication activities. */
		SLATE_ATTRIBUTE(EVisibility, ReplicationActivitiesVisibility)

		/** Show/hide ignored activities. */
		SLATE_ATTRIBUTE(EVisibility, IgnoredActivitiesVisibility)

		/** Show/hide the details area widget. (Not to confuse with widget expansion state) */
		SLATE_ARGUMENT(EVisibility, DetailsAreaVisibility)

		/** True to scroll the list down automatically (unless the user manually scrolled the list). */
		SLATE_ARGUMENT(bool, IsAutoScrollEnabled)

		/** Show/hide a message overlay above the activities list explaining why no activities are displayed. */
		SLATE_ATTRIBUTE(FText, NoActivitiesReasonText)
	
		/** Optional snapshot to restore column visibilities with */
		SLATE_ARGUMENT(FColumnVisibilitySnapshot, ColumnVisibilitySnapshot)
		/** Called whenever the column visibility changes and should be saved */
		SLATE_EVENT(UE::ConcertSharedSlate::FSaveColumnVisibilitySnapshot, SaveColumnVisibilitySnapshot)
	

		/** Optional. How the activities may be selected */
		SLATE_ARGUMENT(TOptional<ESelectionMode::Type>, SelectionMode)

		/** Optional. Used by non-editor instances to provide reflection data about transaction activities. */
		SLATE_ARGUMENT(TSharedPtr<UE::ConcertSharedSlate::IConcertReflectionDataProvider>, UndoHistoryReflectionProvider)

		/** Optional. Whether to reduce focus to activities by darkening when the activity is muted (default: true). */
		SLATE_ARGUMENT(bool, DarkenMutedActivities)
	SLATE_END_ARGS();
	
	/**
	 * Construct the recovery widget.
	 * @param InArgs The widgets arguments and attributes.
	 */
	UE_API void Construct(const FArguments& InArgs);

	/** Requests and consumes more activities if needed and/or possible. */
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/** Returns the activity selected or null if none is selected. */
	UE_API TSharedPtr<FConcertSessionActivity> GetSelectedActivity() const;
	/** Returns all selected activities */
	UE_API TArray<TSharedPtr<FConcertSessionActivity>> GetSelectedActivities() const;

	/** Returns the total number of activities currently stored (no filter applied). */
	int32 GetTotalActivityNum() const { return AllActivities.Num(); }

	/** Returns the number of activities shown. */
	int32 GetDisplayedActivityNum() const { return Activities.Num(); }

	/** Returns the number of activities marked as 'ignored'. */
	int32 GetIgnoredActivityNum() const { return IgnoredActivityNum; }

	/** Returns the most recent activity available, ignoring the current filter. */
	UE_API TSharedPtr<FConcertSessionActivity> GetMostRecentActivity() const;

	/** Returns true if the column names is the last one (most right one). */
	UE_API bool IsLastColumn(const FName& ColumnId) const;

	/** Clears all activities displayed. */
	UE_API void ResetActivityList();

	/** Append an activity to the view. Used to populate the view from a live session. */
	UE_API void Append(TSharedPtr<FConcertSessionActivity> Activity);

	/** Request the view to refresh. */
	UE_API void RequestRefresh();

	/** Asks the view to update the text search filter. */
	UE_API FText UpdateTextFilter(const FText& InFilterText);
	
	UE_API void OnColumnVisibilitySettingsChanged(const FColumnVisibilitySnapshot& ColumnSnapshot);
	
	const TAttribute<FText>& GetHighlightText() const { return HighlightText; }
	const TAttribute<ETimeFormat>& GetTimeFormat() const { return TimeFormat; }
	TOptional<FConcertClientInfo> GetClientInfo(const FGuid& Guid) const { return GetActivityUserFn.IsBound() ? GetActivityUserFn.Execute(Guid) : TOptional<FConcertClientInfo>{}; }
	TSharedPtr<SHeaderRow> GetHeaderRow() const { return HeaderRow; }

	const TArray<TSharedPtr<FConcertSessionActivity>>& GetActivities() const { return Activities; }
	void SetSelectedActivities(const TArray<TSharedPtr<FConcertSessionActivity>>& ActivitiesToSelect)
	{
		ActivityView->ClearSelection();
		ActivityView->SetItemSelection(ActivitiesToSelect, true);
	}

private:

	// Widget creation
	UE_API TSharedRef<SHeaderRow> CreateHeaderRow(const FArguments& InArgs);
	UE_API TSharedRef<ITableRow> OnGenerateActivityRowWidget(TSharedPtr<FConcertSessionActivity> Item, const TSharedRef<STableViewBase>& OwnerTable);
	UE_API void FetchActivities();

	// Filtering
	UE_API EConcertActivityFilterFlags QueryActiveActivityFilters() const;
	UE_API void OnActivityFilterUpdated();
	UE_API bool PassesFilters(const FConcertSessionActivity& Activity);

	// List view
	UE_API void OnListViewScrolled(double InScrollOffset);
	UE_API void OnListViewSelectionChanged(TSharedPtr<FConcertSessionActivity> InActivity, ESelectInfo::Type SelectInfo);

	// Search
	UE_API void PopulateSearchStrings(const FConcertSessionActivity& Item, TArray<FString>& OutSearchStrings);

	// Details
	UE_API void OnDetailsAreaExpansionChanged(bool bExpanded);
	UE_API void UpdateDetailArea(TSharedPtr<FConcertSessionActivity> InSelectedActivity);
	UE_API void UpdateDetailsByEventPayload(TSharedPtr<FConcertSessionActivity> InSelectedActivity);
	UE_API void UpdateDetailsByRequest(TSharedPtr<FConcertSessionActivity> InSelectedActivity);
	
	UE_API void SetDetailsPanelVisibility(const SWidget* VisiblePanel);
	UE_API void DisplayTransactionDetails(const FConcertSessionActivity& Activity, const FConcertTransactionEventBase& InTransaction);
	UE_API void DisplayPackageDetails(const FConcertSessionActivity& Activity, int64 PackageRevision, const FConcertPackageInfo& PackageInfo);
	
	UE_API FText GetNoDetailsText() const;
	EVisibility GetDetailAreaVisibility() const { return DetailsAreaVisibility; }
	SSplitter::ESizeRule GetDetailsAreaSizeRule() const { return bDetailsAreaExpanded ? SSplitter::ESizeRule::FractionOfParent : SSplitter::ESizeRule::SizeToContent; }
	
	/** Invoked after all widget ticks get processed. Used to check if the user scrolled or not. */
	UE_API void OnPostTick(float);

private:
	
	/** List of all activities (including the filtered out ones). */
	TArray<TSharedPtr<FConcertSessionActivity>> AllActivities;

	/** List of currently displayed activities. */
	TArray<TSharedPtr<FConcertSessionActivity>> Activities;

	/** The list view widget displaying the activities. */
	TSharedPtr<SListView<TSharedPtr<FConcertSessionActivity>>> ActivityView;

	/** Provides reflection data for SUndoHistoryDetails for non-editor builds. */
	TOptional<TSharedPtr<UE::ConcertSharedSlate::IConcertReflectionDataProvider>> ConcertReflectionDataProvider; 

	/** Used to overlay a widget over a column widget (add an extra layer above the normal one) */
	FMakeColumnOverlayWidgetFunc MakeColumnOverlayWidgetFn;

	/** If bound, invoked when an item in the table is right-click and we are supposed to show a menu of actions. */
	FOnContextMenuOpening OnContextMenuOpening;

	/** Whether to darken muted activities */
	bool bDarkenMutedActivities = true;

	/** Returns which text should be highlighted. */
	TAttribute<FText> HighlightText;

	/** Whether the time should be displayed as relative (9 seconds ago) or as absolute (July 10, 2019 - 10:20:10) */
	TAttribute<ETimeFormat> TimeFormat;

	/** Whether the join/leave session activities are displayed. */
	TAttribute<EVisibility> ConnectionActivitiesVisibility;

	/** Whether the lock/unlock activities are displayed.*/
	TAttribute<EVisibility> LockActivitiesVisibility;

	/** Whether the package activities are displayed.*/
	TAttribute<EVisibility> PackageActivitiesVisibility;

	/** Whether the transaction activities are displayed.*/
	TAttribute<EVisibility> TransactionActivitiesVisibility;
	
	/** Whether the replication activities are displayed.*/
	TAttribute<EVisibility> ReplicationActivitiesVisibility;
	
	/** Whether the ignored activities are displayed.*/
	TAttribute<EVisibility> IgnoredActivitiesVisibility;

	/** The number of activities flagged as 'ignored' within AllActivities array.*/
	int32 IgnoredActivityNum = 0;

	/** Whether the auto-scrolling is desired (scroll to bottom automatically unless the user scrolls somewhere else). */
	bool bAutoScrollDesired = false;

	/** Whether the user is scrolling (deactivate auto-scrolling). */
	bool bUserScrolling = false;

	/** Defines which activity types are currently filtered out from the view. */
	EConcertActivityFilterFlags ActiveFilterFlags = EConcertActivityFilterFlags::ShowAll;

	/** Used to fetch more activities from an abstract source. Usually mutually exclusive with Append() function. May not be bound. */
	FFetchActivitiesFunc FetchActivitiesFn;
	/** Used to map an activity endpoint ID to a client. May not be bound. */
	FGetActivityClientInfoFunc GetActivityUserFn;
	/** Used to get the transaction event to display the selected transaction activity details. May not be bound. */
	FGetTransactionEvent GetTransactionEventFn;
	/** Used to get the package event to display the selected package activity details. May not be bound. */
	FGetPackageEvent GetPackageEventFn;

	/** The number of activities to request when scrolling down to request activities on demand. Used with FetchActivitiesFn() for paging. */
	static constexpr int32 ActivitiesPerRequest = 128;

	/** The current desired amount of activities to display. Used for paging with FetchActivitiesFn(). It will grow when the user scrolls down. */
	int32 DesiredActivitiesCount = ActivitiesPerRequest;

	/** True once the activity provider function (FetchActivitiesFn) returns true. */
	bool bAllActivitiesFetched = false;
	
	/** Utility class used to tokenize and match text displayed in the list view. */
	TSharedPtr<TTextFilter<const FConcertSessionActivity&>> SearchTextFilter;

	TSharedPtr<SHeaderRow> HeaderRow;

	/** Flag used to suspend saving of the column visibility while OnColumnVisibilitySettingsChanged is active to prevent infinite event recursion. */
	bool bIsUpdatingColumnVisibility = false;
	
	/** The expandable area under which the activity details are displayed. */
	TSharedPtr<SExpandableArea> ExpandableDetails;

	/** The widget displaying transaction details (if detail area is expanded) when a transaction activity is selected. */
	TSharedPtr<SUndoHistoryDetails> TransactionDetailsPanel;

	/** The widget displaying package details (if detail area is expanded) when a package activity is selected.*/
	TSharedPtr<SPackageDetails> PackageDetailsPanel;

	/** The widget saying they are no detail available (if detail area is expanded) when the activity has no details or no activity is selected. */
	TSharedPtr<SBox> NoDetailsPanel;

	/** The widget displayed when details of a partially sync activity (if detail area is expanded) are being fetched from the server. */
	TSharedPtr<SWidget> LoadingDetailsPanel;

	/** Indicate if the details area should be displayed or not. */
	EVisibility DetailsAreaVisibility = EVisibility::Collapsed;

	/** Keeps the expanded status of the details area. */
	bool bDetailsAreaExpanded = false;

	/** Indicate whether the activity list view was scrolled during a frame. Used to correctly detect auto-scrolling in special cases. */
	bool bActivityViewScrolled = false;
};

/**
 * Manages the various options exposed by SConcertSessionActivities widget such as enabling/disabling filtering,
 * changing the time format, etc.
 */
class FConcertSessionActivitiesOptions : public TSharedFromThis<FConcertSessionActivitiesOptions>
{
public:
	
	DECLARE_DELEGATE_OneParam(FExtendContextMenu, FMenuBuilder&)

	/** Makes a status bar widget displaying the activity shown and the standard view options button. */
	UE_API TSharedRef<SWidget> MakeStatusBar(TAttribute<int32> Total, TAttribute<int32> Displayed);
	
	/** Makes a standard View Options widget, displaying the eye ball icon and showing the possible options. */
	UE_API TSharedRef<SWidget> MakeViewOptionsComboButton(TOptional<FExtendContextMenu> ExtendMenu = {});
	/** Returns a Menu widgets containing the available options. */
	UE_API TSharedRef<SWidget> MakeMenuWidget(TOptional<FExtendContextMenu> ExtendMenu = {});

	/** Returns the time format option. */
	ETimeFormat GetTimeFormat() const { return bDisplayRelativeTime ? ETimeFormat::Relative : ETimeFormat::Absolute; }
	/** Returns whether the connection activities are listed in SConcertSessionActivities. */
	EVisibility GetConnectionActivitiesVisibility() const { return bDisplayConnectionActivities ? EVisibility::Visible : EVisibility::Hidden; }
	/** Returns whether the lock activities are listed in SConcertSessionActivities. */
	EVisibility GetLockActivitiesVisibility() const { return bDisplayLockActivities ? EVisibility::Visible : EVisibility::Hidden; }
	/** Returns whether the package activities are listed in SConcertSessionActivities. */
	EVisibility GetPackageActivitiesVisibility() const { return bDisplayPackageActivities ? EVisibility::Visible : EVisibility::Hidden; }
	/** Returns whether the transaction activities are listed in SConcertSessionActivities. */
	EVisibility GetTransactionActivitiesVisibility() const { return bDisplayTransactionActivities ? EVisibility::Visible : EVisibility::Hidden; }
	/** Returns whether the ignored activities are listed in SConcertSessionActivities. */
	EVisibility GetReplicationActivitiesVisibility() const { return bDisplayReplicationActivities ? EVisibility::Visible : EVisibility::Hidden; }
	/** Returns whether the ignored activities are listed in SConcertSessionActivities. */
	EVisibility GetIgnoredActivitiesVisibility() const { return bDisplayIgnoredActivities ? EVisibility::Visible : EVisibility::Hidden; }

	/** Invoked when an options is togged from the displayed menu widget. */
	UE_API void OnOptionToggled(const FName CheckBoxId);

	/** Enables the 'connection activity' filter check box. (Show Connection Activities). */
	bool bEnableConnectionActivityFiltering = true;
	/** Enables the 'lock activity' filter check box. (Show Lock Activities).*/
	bool bEnableLockActivityFiltering = true;
	/** Enables the 'package activity' filter check box (Show Package Activities). */
	bool bEnablePackageActivityFiltering = true;
	/** Enables the 'transaction activity' filter check box (Show Transaction Activities). */
	bool bEnableTransactionActivityFiltering = true;
	/** Enables the 'replication activity' filter check box (Show Replication Activities). */
	bool bEnableReplicationActivityFiltering = true;
	/** Enables the 'ignored activity' filter check box (Show Unrecoverable Activities). */
	bool bEnableIgnoredActivityFiltering = false;
	/** Controls whether the time is displayed as absolute or relative. (Display Relative Time). */
	bool bDisplayRelativeTime = true;
	/** If connection filtering is enabled, controls whether connection activities are filtered out. */
	bool bDisplayConnectionActivities = false;
	/** If lock filtering is enabled, controls whether lock activities are filtered out. */
	bool bDisplayLockActivities = false;
	/** If package filtering is enabled, controls whether package activities are filtered out. */
	bool bDisplayPackageActivities = true;
	/** If transaction filtering is enabled, controls whether transaction activities are filtered out.*/
	bool bDisplayTransactionActivities = true;
	/** If replication filtering is enabled, controls whether replication activities are filtered out. */
	bool bDisplayReplicationActivities = false;
	/** If ignored activity filtering is enabled, controls whether ignored activities are filtered out.*/
	bool bDisplayIgnoredActivities = false;
};

#undef UE_API
