// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ConcertMessageData.h"
#include "IConcertSessionBrowserController.h"
#include "Session/Browser/Items/ConcertSessionTreeItem.h"

#include "Internationalization/Regex.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Misc/AsyncTaskNotification.h"
#include "SNegativeActionButton.h"
#include "SPositiveActionButton.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"

namespace ConcertBrowserUtils
{
	// Defines the sessions list view column tag names.
	static const FName SessionColName(TEXT("Session"));
	static const FName ServerColName(TEXT("Server"));
	static const FName ProjectColName(TEXT("Project"));
	static const FName VersionColName(TEXT("Version"));
	static const FName LastModifiedColName(TEXT("LastModified"));

	// Name of the filter box in the View option.
	static const FName LastModifiedCheckBoxMenuName(TEXT("LastModified"));
	static const FName ActiveSessionsCheckBoxMenuName(TEXT("ActiveSessions"));
	static const FName ArchivedSessionsCheckBoxMenuName(TEXT("ArchivedSessions"));
	static const FName DefaultServerCheckBoxMenuName(TEXT("DefaultServer"));

	// The awesome font used to pick the icon displayed in the session list view 'Icon' column.
	static const FName IconColumnFontName(TEXT("FontAwesome.9"));

	/** Utility function used to create buttons displaying only an icon (using a brush) */
	inline TSharedRef<SWidget> MakeIconButton(const TAttribute<const FSlateBrush*>& Icon, const TAttribute<FText>& Tooltip, const TAttribute<bool>& EnabledAttribute, const FOnClicked& OnClicked, const TAttribute<EVisibility>& Visibility = EVisibility::Visible)
	{
		return SNew(SButton)
			.OnClicked(OnClicked)
			.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
			.ToolTipText(Tooltip)
			.Visibility(Visibility)
			.IsEnabled(EnabledAttribute)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SImage)
				.Image(Icon)
				.ColorAndOpacity(FSlateColor::UseForeground())
			];
	}

	/** Utility function used to create buttons displaying a positive action*/
	inline TSharedRef<SWidget> MakePositiveActionButton(const TAttribute<const FSlateBrush*>& Icon, const TAttribute<FText>& Tooltip, const TAttribute<bool>& EnabledAttribute, const FOnClicked& OnClicked, const TAttribute<EVisibility>& Visibility = EVisibility::Visible)
	{
		return SNew(SPositiveActionButton)
			.OnClicked(OnClicked)
			.ToolTipText(Tooltip)
			.Visibility(Visibility)
			.IsEnabled(EnabledAttribute)
			.Icon(Icon);
	}

	/** Utility function used to create buttons displaying a negative action */
	inline TSharedRef<SWidget> MakeNegativeActionButton(const TAttribute<const FSlateBrush*>& Icon, const TAttribute<FText>& Tooltip, const TAttribute<bool>& EnabledAttribute, const FOnClicked& OnClicked, const TAttribute<EVisibility>& Visibility = EVisibility::Visible)
	{
		return SNew(SNegativeActionButton)
			.OnClicked(OnClicked)
			.ToolTipText(Tooltip)
			.Visibility(Visibility)
			.IsEnabled(EnabledAttribute)
			.Icon(Icon);
	}

	/** Returns the tooltip shown when hovering the triangle with an exclamation icon when a server doesn't validate the version requirements. */
	inline FText GetServerVersionIgnoredTooltip()
	{
#define LOCTEXT_NAMESPACE "SConcertBrowser"
		return LOCTEXT("ServerIgnoreSessionRequirementsTooltip", "Careful this server won't verify that you have the right requirements before you join a session");
#undef LOCTEXT_NAMESPACE
	}

	/** Create a widget displaying the triangle with an exclamation icon in case the server flags include IgnoreSessionRequirement. */
	inline TSharedRef<SWidget> MakeServerVersionIgnoredWidget(EConcertServerFlags InServerFlags)
	{
		return SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("NoBorder"))
			.ColorAndOpacity(FAppStyle::Get().GetWidgetStyle<FButtonStyle>("FlatButton.Warning").Normal.TintColor.GetSpecifiedColor())
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Warning"))
				.ToolTipText(GetServerVersionIgnoredTooltip())
				.Visibility((InServerFlags & EConcertServerFlags::IgnoreSessionRequirement) != EConcertServerFlags::None ? EVisibility::Visible : EVisibility::Collapsed)
			];
	}

	/** Helper function to split an array of FConcertSessionItems into multiple requests and execute*/
	inline void RequestItemDeletion(IConcertSessionBrowserController& Controller, const TArray<TSharedPtr<FConcertSessionTreeItem>>& SessionItems)
	{
		using FServerAdminEndpointId = FGuid;
		using FSessionId = FGuid;
			
		TMap<FServerAdminEndpointId, TArray<FSessionId>> Requests;
		for (const TSharedPtr<FConcertSessionTreeItem>& Item : SessionItems)
		{
			Requests.FindOrAdd(Item->ServerAdminEndpointId).Add(Item->SessionId);
		}
		for (auto RequestIt = Requests.CreateConstIterator(); RequestIt; ++RequestIt)
		{
			Controller.DeleteSessions(RequestIt->Key, RequestIt->Value);
		}
	}
}
