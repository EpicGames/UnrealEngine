// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertFrontendUtils.h"

#include "UObject/Object.h"
#include "ConcertSessionBrowserSettings.generated.h"

/** Serializes the multi-user session browser settings like the active filters. */
UCLASS(MinimalAPI, config=Editor)
class UConcertSessionBrowserSettings : public UObject
{
	GENERATED_BODY()

public:
	UConcertSessionBrowserSettings() {}

	UPROPERTY(config, EditAnywhere, Category="Multi-User Session Browser")
	ETimeFormat LastModifiedTimeFormat = ETimeFormat::Absolute;
	
	UPROPERTY(config, EditAnywhere, Category="Multi-User Session Browser")
	bool bShowActiveSessions = true;

	UPROPERTY(config, EditAnywhere, Category="Multi-User Session Browser")
	bool bShowArchivedSessions = true;

	UPROPERTY(config, EditAnywhere, Category="Multi-User Session Browser")
	bool bShowDefaultServerSessionsOnly = false;
};

