// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertMessageData.h"

/**
 * Runs and cache network queries for the UI. In the model-view-controller pattern, this class acts like the controller. Its purpose
 * is to keep the UI code as decoupled as possible from the API used to query it. It encapsulate the asynchronous code and provide a
 * simpler API to the UI.
 */
class IConcertSessionBrowserController
{
public:

	struct FActiveSessionInfo
	{
		FConcertServerInfo ServerInfo;
		FConcertSessionInfo SessionInfo;
		TArray<FConcertSessionClientInfo> Clients;

		FActiveSessionInfo(FConcertServerInfo ServerInfo, FConcertSessionInfo SessionInfo, TArray<FConcertSessionClientInfo> Clients = {})
			: ServerInfo(MoveTemp(ServerInfo))
			, SessionInfo(MoveTemp(SessionInfo))
			, Clients(MoveTemp(Clients))
		{}
	};

	struct FArchivedSessionInfo
	{
		FConcertServerInfo ServerInfo;
		FConcertSessionInfo SessionInfo;

		FArchivedSessionInfo(FConcertServerInfo ServerInfo, FConcertSessionInfo SessionInfo)
			: ServerInfo(MoveTemp(ServerInfo))
			, SessionInfo(MoveTemp(SessionInfo))
		{}
	};

public:
	/** Returns the latest list of server known to this controller. */
	virtual TArray<FConcertServerInfo> GetServers() const = 0;

	/** Returns the latest list of active sessions known to this controller. */
	virtual TArray<FActiveSessionInfo> GetActiveSessions() const = 0;

	/** Returns the latest list of archived sessions known to this controller. */
	virtual TArray<FArchivedSessionInfo> GetArchivedSessions() const = 0;

	/** Returns the active sessions info corresponding to the specified parameters. Used to display the sessions details. */
	virtual TOptional<FConcertSessionInfo> GetActiveSessionInfo(const FGuid& AdminEndpoint, const FGuid& SessionId) const = 0;

	/** Returns the archived sessions info corresponding to the specified parameters. Used to display the sessions details. */
	virtual TOptional<FConcertSessionInfo> GetArchivedSessionInfo(const FGuid& AdminEndpoint, const FGuid& SessionId) const = 0;

	virtual void CreateSession(const FGuid& ServerAdminEndpointId, const FString& SessionName, const FString& ProjectName) = 0;
	virtual void ArchiveSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& ArchiveName, const FConcertSessionFilter& SessionFilter) = 0;
	virtual void RestoreSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& RestoredName, const FConcertSessionFilter& SessionFilter) = 0;
	virtual void RenameActiveSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& NewName) = 0;
	virtual void RenameArchivedSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& NewName) = 0;
	virtual bool CanRenameActiveSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) const = 0;
	virtual bool CanRenameArchivedSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) const = 0;
	virtual void DeleteSessions(const FGuid& ServerAdminEndpointId, const TArray<FGuid>& SessionIds) = 0;
	virtual bool CanDeleteActiveSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) const = 0;
	virtual bool CanDeleteArchivedSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) const = 0;
	virtual bool CanEverCreateSessions() const = 0;

	virtual ~IConcertSessionBrowserController() = default;
};
