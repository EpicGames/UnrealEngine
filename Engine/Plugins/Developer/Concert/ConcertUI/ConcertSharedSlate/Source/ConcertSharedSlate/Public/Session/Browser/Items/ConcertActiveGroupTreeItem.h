// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertGroupTreeItem.h"
#include "ConcertSessionTreeItem.h"

/** Groups several active sessions */
class FConcertActiveGroupTreeItem : public FConcertGroupTreeItem
{
	using Super = FConcertGroupTreeItem;
public:
	
	FConcertActiveGroupTreeItem(FGetSessions GetSessionsFunc)
		: Super(MoveTemp(GetSessionsFunc), { FConcertSessionTreeItem::EType::ActiveSession, FConcertSessionTreeItem::EType::NewSession, FConcertSessionTreeItem::EType::SaveSession })
	{}
};
