// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertGroupTreeItem.h"

/** Groups archived sessions */
class FConcertArchivedGroupTreeItem : public FConcertGroupTreeItem
{
	using Super = FConcertGroupTreeItem;
public:
	
	FConcertArchivedGroupTreeItem(FGetSessions GetSessionsFunc)
		: Super(MoveTemp(GetSessionsFunc), { FConcertSessionTreeItem::EType::ArchivedSession, FConcertSessionTreeItem::EType::RestoreSession })
	{}
};
