// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertSessionTreeItem.h"
#include "ConcertTreeItem.h"

/** Groups several FConcertSessionTreeItem */
class FConcertGroupTreeItem : public FConcertTreeItem
{
public:

	using FGetSessions = TFunction<TArray<TSharedPtr<FConcertSessionTreeItem>>()>;

	FConcertGroupTreeItem(FGetSessions GetSessionsFunc, TSet<FConcertSessionTreeItem::EType> AllowedTypes)
		: GetSessionsFunc(MoveTemp(GetSessionsFunc))
		, AllowedTypes(MoveTemp(AllowedTypes))
	{}

	virtual void GetChildren(TArray<TSharedPtr<FConcertTreeItem>>& OutChildren) const override
	{
		const TArray<TSharedPtr<FConcertSessionTreeItem>>& Items = GetSessionsFunc(); 
		for (const TSharedPtr<FConcertSessionTreeItem>& Item : Items)
		{
			if (AllowedTypes.Contains(Item->Type))
			{
				OutChildren.Add(Item);
			}
		}
	}

protected:
	
	const FGetSessions GetSessionsFunc;
	const TSet<FConcertSessionTreeItem::EType> AllowedTypes;
};
