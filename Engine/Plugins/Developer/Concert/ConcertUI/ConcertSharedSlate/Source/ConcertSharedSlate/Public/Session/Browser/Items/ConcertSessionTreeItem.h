// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertMessageData.h"
#include "ConcertTreeItem.h"

/** Signal emitted when a session name text field should enter in edit mode. */
DECLARE_MULTICAST_DELEGATE(FOnBeginEditConcertSessionNameRequest)

/** An item that represents a session, i.e. has session info. */
class FConcertSessionTreeItem : public FConcertTreeItem
{
public:
	enum class EType
	{
		None,
		NewSession,				// Editable item to enter a session name and a pick a server.
		RestoreSession,			// Editable item to name the restored session.
		SaveSession,			// Editable item to name the archive.
		ActiveSession,			// Read-only item representing an active session.
		ArchivedSession,		// Read-only item representing an archived session.
	};

	FConcertSessionTreeItem(EType Type = EType::None,
		const FGuid& ServerAdminEndpointId = {},
		const FGuid& SessionId = {},
		const FString& SessionName = {},
		const FString& ServerName = {},
		const FString& ProjectName = {},
		const FString& ProjectVersion = {},
		EConcertServerFlags ServerFlags = {},
		const FDateTime& LastModified = {},
		const FOnBeginEditConcertSessionNameRequest& OnBeginEditSessionNameRequest = {}
		)
		: Type(Type)
		, ServerAdminEndpointId(ServerAdminEndpointId)
		, SessionId(SessionId)
		, SessionName(SessionName)
		, ServerName(ServerName)
		, ProjectName(ProjectName)
		, ProjectVersion(ProjectVersion)
		, ServerFlags(ServerFlags)
		, LastModified(LastModified)
		, OnBeginEditSessionNameRequest(OnBeginEditSessionNameRequest)
	{}

	bool operator==(const FConcertSessionTreeItem& Other) const
	{
		return Type == Other.Type && ServerAdminEndpointId == Other.ServerAdminEndpointId && SessionId == Other.SessionId;
	}

	FConcertSessionTreeItem MakeCopyAsType(EType NewType) const
	{
		FConcertSessionTreeItem NewItem = *this;
		NewItem.Type = NewType;
		return NewItem;
	}

	virtual void GetChildren(TArray<TSharedPtr<FConcertTreeItem>>& OutChildren) const override {}

	EType Type = EType::None;
	FGuid ServerAdminEndpointId;
	FGuid SessionId;
	FString SessionName;
	FString ServerName;
	FString ProjectName;
	FString ProjectVersion;
	EConcertServerFlags ServerFlags = EConcertServerFlags::None;
	
	FDateTime LastModified;

	FOnBeginEditConcertSessionNameRequest OnBeginEditSessionNameRequest; // Emitted when user press 'F2' or select 'Rename' from context menu.
};


