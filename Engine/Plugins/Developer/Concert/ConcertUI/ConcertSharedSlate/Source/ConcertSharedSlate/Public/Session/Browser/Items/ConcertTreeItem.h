// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** Base class for everything displayed in session view */
class FConcertTreeItem
{
public:

	virtual void GetChildren(TArray<TSharedPtr<FConcertTreeItem>>& OutChildren) const = 0;
	
	virtual ~FConcertTreeItem() = default;
};
