// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertHeaderRowUtils.h"

#include "Misc/TextFilter.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STreeView.h"

#define UE_API CONCERTSHAREDSLATE_API

class FConcertActiveGroupTreeItem;
class FConcertArchivedGroupTreeItem;
class FConcertTreeItem;
class FConcertSessionTreeItem;
class FExtender;
class IConcertSessionBrowserController;
class ITableRow;
class UConcertSessionBrowserSettings;
class SGridPanel;
class STableViewBase;

struct FConcertSessionClientInfo;
struct FConcertSessionInfo;

DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FExtendSessionTable, const TSharedRef<SWidget>& /* TableView */);
DECLARE_DELEGATE_OneParam(FExtenderDelegate, FExtender&);
DECLARE_DELEGATE_TwoParams(FExtendSessionContextMenu, const TSharedPtr<FConcertSessionTreeItem>&, FExtender&)
DECLARE_DELEGATE_OneParam(FSessionDelegate, const TSharedPtr<FConcertSessionTreeItem>& /*Session*/);
DECLARE_DELEGATE_OneParam(FSessionListDelegate, const TArray<TSharedPtr<FConcertSessionTreeItem>>& /*Sessions*/);
DECLARE_DELEGATE_RetVal_OneParam(bool, FCanRemoveSessions, const TArray<TSharedPtr<FConcertSessionTreeItem>>& /* SessionItem */);

/**
 * Enables the user to browse/search/filter/sort active and archived sessions, create new session,
 * archive active sessions, restore archived sessions, join a session and open the settings dialog.
 */
class SConcertSessionBrowser : public SCompoundWidget
{
public:
	
	struct ControlButtonExtensionHooks
	{
		/** Contains: Create Session */
		static UE_API const FName BeforeSeparator;
		/** Just separates the two */
		static UE_API const FName Separator;
		/** Contains: Restore, Archive, Delete */
		static UE_API const FName AfterSeparator;
	};

	struct SessionContextMenuExtensionHooks
	{
		/** Contains: Archive (ActiveSession), Restore (ArchivedSession), Rename, Delete  */
		static UE_API const FName ManageSession;
	};
	
	SLATE_BEGIN_ARGS(SConcertSessionBrowser) { }

	/** Optional name of the default session - relevant for highlighting */
	SLATE_ATTRIBUTE(FString, DefaultSessionName)
	/** Optional url of the default server - relevant for highlighting */
	SLATE_ATTRIBUTE(FString, DefaultServerURL)

	/** Used during construction to override how the session table view is created, e.g. to embed it into an overlay */
	SLATE_EVENT(FExtendSessionTable, ExtendSessionTable)
	/** Extends the buttons to the left of the search bar */
	SLATE_EVENT(FExtenderDelegate, ExtendControllButtons)
	/** Extends the menu when the user right-clicks a session */
	SLATE_EVENT(FExtendSessionContextMenu, ExtendSessionContextMenu)
	/** Custom slot placed to the right the control button in the top-most bar */
	SLATE_NAMED_SLOT(FArguments, RightOfControlButtons)
	
	/** Called when a live or archived session is clicked */
	SLATE_EVENT(FSessionDelegate, OnSessionClicked)
	/** Called when a live session is double-clicked */
	SLATE_EVENT(FSessionDelegate, OnLiveSessionDoubleClicked)
	/** Called when an archived session is double-clicked. If unset, the default behaviour is to rename the session. */
	SLATE_EVENT(FSessionDelegate, OnArchivedSessionDoubleClicked)
	/** Called after a user has requested to delete a session */
	SLATE_EVENT(FSessionListDelegate, PostRequestedDeleteSession)

	/** Ask the user to confirm archiving - most obvious implementation is showing a dialog box */
	SLATE_EVENT(FCanRemoveSessions, AskUserToDeleteSessions)

	/** Optional snapshot to restore column visibilities with */
	SLATE_ARGUMENT(FColumnVisibilitySnapshot, ColumnVisibilitySnapshot)
	/** Called whenever the column visibility changes and should be saved */
	SLATE_EVENT(UE::ConcertSharedSlate::FSaveColumnVisibilitySnapshot, SaveColumnVisibilitySnapshot)
	
	SLATE_END_ARGS();

	/**
	* Constructs the Browser.
	* @param InArgs The Slate argument list.
	* @param InController The controller used to send queries from the UI - represents controller in mode-view-controller pattern.
	* @param[in,out] InSearchText The text to set in the search box and to remember (as output). Cannot be null.
	*/
	UE_API void Construct(const FArguments& InArgs, TSharedRef<IConcertSessionBrowserController> InController, TSharedPtr<FText> InSearchText);

	//~ Begin SWidget Interface
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget Interface
	
	UE_API void RefreshSessionList();
	bool HasAnySessions() const { return Sessions.Num() > 0; }
	UE_API TArray<TSharedPtr<FConcertSessionTreeItem>> GetSelectedItems() const;

	bool IsNewButtonEnabled() const { return IsNewButtonEnabledInternal(); }
	bool IsRestoreButtonEnabled() const { return IsRestoreButtonEnabledInternal(); }
	bool IsArchiveButtonEnabled() const { return IsArchiveButtonEnabledInternal(); }
	bool IsRenameButtonEnabled() const { return IsRenameButtonEnabledInternal(); }
	bool IsDeleteButtonEnabled() const { return IsDeleteButtonEnabledInternal(); }
	
	/** Adds row for creating new session. Exposed for other widgets, e.g. discovery overlay to create a new session. */
	void InsertNewSessionEditableRow() { InsertNewSessionEditableRowInternal(); }
	/** Creates row under the given (archived) session with which session can be restored. */
	void InsertRestoreSessionAsEditableRow(const TSharedPtr<FConcertSessionTreeItem>& ArchivedItem) { InsertRestoreSessionAsEditableRowInternal(ArchivedItem); }
	
private:

	enum class EInsertPosition
	{
		AtBeginning,
		AfterParent
	};
	
	// Layout the 'session|details' split view.
	UE_API TSharedRef<SWidget> MakeBrowserContent(const FArguments& InArgs);
	
	UE_API void OnGetChildren(TSharedPtr<FConcertTreeItem> InItem, TArray<TSharedPtr<FConcertTreeItem>>& OutChildren);

	// Layout the sessions view and controls.
	UE_API TSharedRef<SWidget> MakeControlBar(const FArguments& InArgs);
	UE_API TSharedRef<SWidget> MakeButtonBar(const FArguments& InArgs);
	UE_API TSharedRef<SWidget> MakeSessionTableView(const FArguments& InArgs);
	UE_API TSharedRef<SWidget> MakeSessionTableFooter();
	UE_API TSharedRef<ITableRow> OnGenerateSessionRowWidget(TSharedPtr<FConcertTreeItem> Item, const TSharedRef<STableViewBase>& OwnerTable);

	// Creates row widgets for session list view, validates user inputs and forward user requests for processing to a delegate function implemented by this class.
	UE_API TSharedRef<ITableRow> MakeActiveSessionRowWidget(const TSharedPtr<FConcertSessionTreeItem>& ActiveItem, const TSharedRef<STableViewBase>& OwnerTable);
	UE_API TSharedRef<ITableRow> MakeArchivedSessionRowWidget(const TSharedPtr<FConcertSessionTreeItem>& ArchivedItem, const TSharedRef<STableViewBase>& OwnerTable);
	UE_API TSharedRef<ITableRow> MakeNewSessionRowWidget(const TSharedPtr<FConcertSessionTreeItem>& NewItem, const TSharedRef<STableViewBase>& OwnerTable);
	UE_API TSharedRef<ITableRow> MakeRestoreSessionRowWidget(const TSharedPtr<FConcertSessionTreeItem>& RestoreItem, const TSharedRef<STableViewBase>& OwnerTable);
	UE_API TSharedRef<ITableRow> MakeSaveSessionRowWidget(const TSharedPtr<FConcertSessionTreeItem>& ArchivedItem, const TSharedRef<STableViewBase>& OwnerTable);
	UE_API TSharedRef<ITableRow> MakeGroupRowWidget(const TSharedPtr<FConcertTreeItem>& GroupItem, const TSharedRef<STableViewBase>& OwnerTable); 

	// Creates the contextual menu when right clicking a session list view row.
	UE_API TSharedPtr<SWidget> MakeTableContextualMenu();
	UE_API TSharedRef<SWidget> MakeViewOptionsComboButtonMenu();

	// The buttons above the session view.
	UE_API bool IsNewButtonEnabledInternal() const;
	UE_API bool IsRestoreButtonEnabledInternal() const;
	UE_API bool IsArchiveButtonEnabledInternal() const;
	UE_API bool IsRenameButtonEnabledInternal() const;
	UE_API bool IsDeleteButtonEnabledInternal() const;
	UE_API FReply OnNewButtonClicked();
	UE_API FReply OnRestoreButtonClicked();
	UE_API FReply OnArchiveButtonClicked();
	UE_API FReply OnDeleteButtonClicked();
	UE_API void OnBeginEditingSessionName(TSharedPtr<FConcertSessionTreeItem> Item);

	// Manipulates the sessions view (the array and the UI).
	UE_API void OnSessionSelectionChanged(TSharedPtr<FConcertTreeItem> SelectedSession, ESelectInfo::Type SelectInfo);
	UE_API void OnSessionSelectionChanged(TSharedPtr<FConcertSessionTreeItem> SelectedSession, ESelectInfo::Type SelectInfo);
	UE_API void InsertNewSessionEditableRowInternal();
	UE_API void InsertRestoreSessionAsEditableRowInternal(const TSharedPtr<FConcertSessionTreeItem>& ArchivedItem);
	UE_API void InsertArchiveSessionAsEditableRow(const TSharedPtr<FConcertSessionTreeItem>& ActiveItem);
	UE_API void InsertEditableSessionRow(TSharedPtr<FConcertSessionTreeItem> EditableItem, TSharedPtr<FConcertSessionTreeItem> ParentItem);
	UE_API void RemoveSessionRow(const TSharedPtr<FConcertSessionTreeItem>& Item);

	// Sessions sorting. (Sorts the session view)
	UE_API EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;
	UE_API EColumnSortPriority::Type GetColumnSortPriority(const FName ColumnId) const;
	UE_API void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);
	UE_API void EnsureEditableParentChildOrder();

	// Sessions filtering. (Filters the session view)
	UE_API void OnSearchTextChanged(const FText& InFilterText);
	UE_API void OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type CommitType);
	UE_API void OnFilterMenuChecked(const FName MenuName);
	UE_API void PopulateSearchStrings(const FConcertSessionTreeItem& Item, TArray<FString>& OutSearchStrings) const;
	UE_API bool IsFilteredOut(const FConcertSessionTreeItem& Item) const;
	UE_API FText HighlightSearchText() const;

	// Passes the user requests to FConcertBrowserController.
	UE_API void RequestCreateSession(const TSharedPtr<FConcertSessionTreeItem>& NewItem);
	UE_API void RequestArchiveSession(const TSharedPtr<FConcertSessionTreeItem>& ActiveItem, const FString& ArchiveName);
	UE_API void RequestRestoreSession(const TSharedPtr<FConcertSessionTreeItem>& RestoreItem, const FString& SessionName);
	UE_API void RequestRenameSession(const TSharedPtr<FConcertSessionTreeItem>& RenamedItem, const FString& NewName);
	UE_API void RequestDeleteSessions(const TArray<TSharedPtr<FConcertSessionTreeItem>>& ItemsToDelete);

	UE_API TSharedPtr<IConcertSessionBrowserController> GetController() const;

	UE_API bool IsGroupItem(const TSharedPtr<FConcertTreeItem>& TreeItem) const;

	// Modifying sessions arrays
	UE_API void ResetSessions();
	UE_API void AddSession(TSharedPtr<FConcertSessionTreeItem> Session);
	UE_API void InsertSessionAtBeginning(TSharedPtr<FConcertSessionTreeItem> NewSession);
	UE_API void InsertSessionAfterParent(TSharedPtr<FConcertSessionTreeItem> NewSession, TSharedPtr<FConcertSessionTreeItem> Parent);
	UE_API EInsertPosition InsertSessionAfterParentIfAvailableOrAtBeginning(TSharedPtr<FConcertSessionTreeItem> NewSession, TSharedPtr<FConcertSessionTreeItem> Parent);
	UE_API void RemoveSession(TSharedPtr<FConcertSessionTreeItem> Session);
	UE_API void SortSessionList();

private:
	
	// Gives access to the concert data (servers, sessions, clients, etc).
	TWeakPtr<IConcertSessionBrowserController> Controller;

	// Keeps persistent user preferences, like the filters.
	UConcertSessionBrowserSettings* PersistentSettings = nullptr;

	/** Optional default session name - relevant for highlighting */
	TAttribute<FString> DefaultSessionName;
	/** Optional default server url - relevant for highlighting */
	TAttribute<FString> DefaultServerUrl;
	
	FExtendSessionContextMenu ExtendSessionContextMenu;
	/** Called when a live or archived session is clicked */
	FSessionDelegate OnSessionClicked;
	FSessionDelegate OnLiveSessionDoubleClicked;
	/** If unset, the default behaviour is to rename the session. */
	FSessionDelegate OnArchivedSessionDoubleClicked;
	/** Called after a live or archived session was requested to be deleted (it may or may not have been deleted).*/
	FSessionListDelegate PostRequestedDeleteSession;
	FCanRemoveSessions AskUserToDeleteSessions;

	/** The items displayed in the session list view. It might be filtered and sorted compared to the full list hold by the controller. */
	TArray<TSharedPtr<FConcertSessionTreeItem>> Sessions;
	/** Groups all active sessions under it */
	TSharedPtr<FConcertActiveGroupTreeItem> ActiveSessionsGroupItem;
	/** Groups all archived sessions under it */
	TSharedPtr<FConcertArchivedGroupTreeItem> ArchivedSessionsGroupItem;
	/** ItemSource for SessionsView containing Sessions, ActiveGroup, and ArchivedGroup */
	TArray<TSharedPtr<FConcertTreeItem>> TreeItemSource;

	// The session list view.
	TSharedPtr<STreeView<TSharedPtr<FConcertTreeItem>>> SessionsView;
	TSharedPtr<SHeaderRow> SessionHeaderRow;

	// The item corresponding to a row used to create/archive/restore a session. There is only one at the time
	TSharedPtr<FConcertSessionTreeItem> EditableSessionRow;
	TSharedPtr<FConcertSessionTreeItem> EditableSessionRowParent; // For archive/restore, indicate which element is archived or restored.

	// Sorting.
	EColumnSortMode::Type PrimarySortMode = EColumnSortMode::None;
	EColumnSortMode::Type SecondarySortMode = EColumnSortMode::None;
	FName PrimarySortedColumn;
	FName SecondarySortedColumn;

	// Filtering.
	TSharedPtr<SSearchBox> SearchBox;
	TSharedPtr<TTextFilter<const FConcertSessionTreeItem&>> SearchTextFilter;
	TSharedPtr<FText> SearchedText;
	bool bRefreshSessionFilter = true;
	FString LastDefaultServerUrl;

	// Selected Session Details.
	TSharedPtr<SBorder> SessionDetailsView;
	TSharedPtr<SExpandableArea> DetailsArea;
	TArray<TSharedPtr<FConcertSessionClientInfo>> Clients;
	TSharedPtr<SExpandableArea> ClientsArea;
};

#undef UE_API
