// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SSessionHistory.h"

#include "Async/Future.h"

#define UE_API CONCERTSHAREDSLATE_API

struct FConcertClientInfo;
struct FConcertSyncTransactionEvent;
struct FConcertSyncPackageEventMetaData;
struct FConcertSessionActivity;
class IConcertSyncClient;

/** Acts as controller of SSessionHistory in model-view-component pattern. */
class FAbstractSessionHistoryController : public TSharedFromThis<FAbstractSessionHistoryController>
{
public:

	UE_API FAbstractSessionHistoryController(SSessionHistory::FArguments Arguments = SSessionHistory::FArguments());
	virtual ~FAbstractSessionHistoryController() = default;
	
	/** Fetches activities from the server and updates the list view. */
	UE_API void ReloadActivities();
	
	const TSharedRef<SSessionHistory>& GetSessionHistory() const { return SessionHistory; }

protected:

	/** Gets the latest activities*/
	virtual void GetActivities(int64 MaximumNumberOfActivities, TMap<FGuid, FConcertClientInfo>& EndpointClientInfoMap, TArray<FConcertSessionActivity>& FetchedActivities) const = 0;
	
	/** Returns the specified package event (without the package data itself) if available. */
	virtual bool GetPackageEvent(const FConcertSessionActivity& Activity, FConcertSyncPackageEventMetaData& OutPackageEvent) const = 0;

	/** Returns the specified package event if available. */
	virtual TFuture<TOptional<FConcertSyncTransactionEvent>> GetTransactionEvent(const FConcertSessionActivity& Activity) const = 0;
	
private:

	/** The widget being managed */
	TSharedRef<SSessionHistory> SessionHistory;

	UE_API TSharedRef<SSessionHistory> MakeSessionHistory(SSessionHistory::FArguments Arguments) const;
};

#undef UE_API
