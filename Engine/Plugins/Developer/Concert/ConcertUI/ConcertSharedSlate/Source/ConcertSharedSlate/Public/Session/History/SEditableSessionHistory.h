// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Session/History/SSessionHistory.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API CONCERTSHAREDSLATE_API

class SSessionHistory;
struct FConcertSessionActivity;

struct FCanPerformActionResult
{
	TOptional<FText> DeletionReason;

	static FCanPerformActionResult Yes() { return FCanPerformActionResult(); }
	static FCanPerformActionResult No(FText Reason) { return FCanPerformActionResult(MoveTemp(Reason)); }

	bool CanPerformAction() const { return !DeletionReason.IsSet(); }

	explicit FCanPerformActionResult(TOptional<FText> DeletionReason = {})
		: DeletionReason(DeletionReason)
	{}
};

/** Allows activities in the session history to be deleted. */
class SEditableSessionHistory : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SSessionHistory>, FMakeSessionHistory, SSessionHistory::FArguments)
	DECLARE_DELEGATE_RetVal_OneParam(FCanPerformActionResult, FCanPerformActionOnActivities, const TSet<TSharedRef<FConcertSessionActivity>>& /*Activities*/)
	DECLARE_DELEGATE_OneParam(FRequestActivitiesAction, const TSet<TSharedRef<FConcertSessionActivity>>&)

	SLATE_BEGIN_ARGS(SEditableSessionHistory)
	{}
		SLATE_EVENT(FMakeSessionHistory, MakeSessionHistory)
	
		/** Can selected activities be deleted? */
		SLATE_EVENT(FCanPerformActionOnActivities, CanDeleteActivities)
		/** Delete the selected activities */
		SLATE_EVENT(FRequestActivitiesAction, DeleteActivities)
		/** Can selected activities be muted? */
		SLATE_EVENT(FCanPerformActionOnActivities, CanMuteActivities)
		/** Mute the selected activities */
		SLATE_EVENT(FRequestActivitiesAction, MuteActivities)
		/** Can selected activities be unmuted? */
		SLATE_EVENT(FCanPerformActionOnActivities, CanUnmuteActivities)
		/** Unmute the selected activities */
		SLATE_EVENT(FRequestActivitiesAction, UnmuteActivities)
	
		SLATE_NAMED_SLOT(FArguments, StatusBar)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);
	
	//~ Begin SWidget Interface
	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget Interface
	
private:

	TSharedPtr<SSessionHistory> SessionHistory;

	FCanPerformActionOnActivities CanDeleteActivitiesFunc;
	FRequestActivitiesAction DeleteActivitiesFunc;
	FCanPerformActionOnActivities CanMuteActivitiesFunc;
	FRequestActivitiesAction MuteActivitiesFunc;
	FCanPerformActionOnActivities CanUnmuteActivitiesFunc;
	FRequestActivitiesAction UnmuteActivitiesFunc;

	UE_API TSharedPtr<SWidget> OnContextMenuOpening();
	
	UE_API FReply OnClickDeleteActivitiesButton() const;
	UE_API FText GetDeleteActivitiesToolTip() const;
	UE_API bool IsDeleteButtonEnabled() const;
	
	UE_API FReply OnClickMuteActivitesButton() const;
	UE_API FText GetMuteActivitiesToolTip() const;
	UE_API bool IsMuteButtonEnabled() const;
	
	UE_API FReply OnClickUnmuteActivitesButton() const;
	UE_API FText GetUnmuteActivitiesToolTip() const;
	UE_API bool IsUnmuteButtonEnabled() const;

	UE_API FText GenerateTooltip(const FCanPerformActionOnActivities& CanPerformAction, FText SelectActivityToolTip, FText PerformActionToolTipFmt, FText CannotPerformActionToolTipFmt) const;
};

#undef UE_API
