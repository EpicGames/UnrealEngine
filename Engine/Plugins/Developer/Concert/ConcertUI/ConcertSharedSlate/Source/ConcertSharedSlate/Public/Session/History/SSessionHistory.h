// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Session/Activity/PredefinedActivityColumns.h"
#include "Session/Activity/SConcertSessionActivities.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API CONCERTSHAREDSLATE_API

namespace UE::ConcertSharedSlate
{
	class IConcertReflectionDataProvider;
}

class SSearchBox;
class FConcertSessionActivitiesOptions;

/** Wraps a session table list with a search box and a status bar for visibility settings (show connection activities, etc.). */
class SSessionHistory : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_RetVal_TwoParams(bool, FAllowActivity, const FConcertSyncActivity&, const TStructOnScope<FConcertSyncActivitySummary>&)

	/** Maximum number of activities displayed on screen. */ 
	static constexpr int64 MaximumNumberOfActivities = 1000;
	
	SLATE_BEGIN_ARGS(SSessionHistory)
		: _Columns({
			UE::ConcertSharedSlate::ActivityColumn::AvatarColor(),
			UE::ConcertSharedSlate::ActivityColumn::ClientName(),
			UE::ConcertSharedSlate::ActivityColumn::Operation()
		})
		, _DetailsAreaVisibility(EVisibility::Visible)
		, _SelectionMode(ESelectionMode::Single)
		, _DarkenMutedActivities(true)
	{}
	
		SLATE_EVENT(SConcertSessionActivities::FGetPackageEvent, GetPackageEvent)
		SLATE_EVENT(SConcertSessionActivities::FGetTransactionEvent, GetTransactionEvent)
		/** If bound, invoked when an item in the table is right-click and we are supposed to show a menu of actions. */
		SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)
	
		/** Optional name for only displaying activities that affect certain packages */
		SLATE_ARGUMENT(FName, PackageFilter)
		/** Optional filter for filtering activities out */
		SLATE_EVENT(FAllowActivity, AllowActivity)
	
		/** Optional. The columns to show for each activity; they are shown after date time. Defaults to avatar color, client name, and operation. */
		SLATE_ARGUMENT(TArray<FActivityColumn>, Columns)
		/** Optional. If bound, invoked when generating a row to add an overlay to a column. */
		SLATE_EVENT(SConcertSessionActivities::FMakeColumnOverlayWidgetFunc, OnMakeColumnOverlayWidget)
	
		/** Optional snapshot to restore column visibilities with */
		SLATE_ARGUMENT(FColumnVisibilitySnapshot, ColumnVisibilitySnapshot)
		/** Called whenever the column visibility changes and should be saved */
		SLATE_EVENT(UE::ConcertSharedSlate::FSaveColumnVisibilitySnapshot, SaveColumnVisibilitySnapshot)

		/** Optional. You can override the view options with this. */
		SLATE_ATTRIBUTE(TSharedPtr<FConcertSessionActivitiesOptions>, ViewOptions)
	
		/** Optional. Show/hide the details area widget. Default to visible. */
		SLATE_ARGUMENT(EVisibility, DetailsAreaVisibility)
		
		/** Optional. How the activities may be selected. Default to Single. */
		SLATE_ARGUMENT(ESelectionMode::Type, SelectionMode)
	
		/** Optional. Used by non-editor instances to provide reflection data about transaction activities. */
		SLATE_ARGUMENT(TSharedPtr<UE::ConcertSharedSlate::IConcertReflectionDataProvider>, UndoHistoryReflectionProvider)
	
		/** Optional. Whether to reduce focus to activities by darkening when the activity is muted (default: true). */
		SLATE_ARGUMENT(bool, DarkenMutedActivities)
	
		/** Optional. An area to the left of the search bar intended for adding buttons to. */
		SLATE_NAMED_SLOT(FArguments, SearchButtonArea)
	
	SLATE_END_ARGS()

	/**
	 * Constructs the Session History widget.
	 *
	 * @param InArgs The Slate argument list.
	 */
	UE_API void Construct(const FArguments& InArgs);

	/** Fetches activities from the server and updates the list view. */
	UE_API void ReloadActivities(TMap<FGuid, FConcertClientInfo> EndpointClientInfoMap, TArray<FConcertSessionActivity> FetchedActivities);
	
	/** Callback for handling the a new or updated activity item. */ 
	UE_API void HandleActivityAddedOrUpdated(const FConcertClientInfo& InClientInfo, const FConcertSyncActivity& InActivity, const FStructOnScope& InActivitySummary);

	bool IsLastColumn(FName ColumnId) const { return ActivityListView->IsLastColumn(ColumnId); }
	
	UE_API TSet<TSharedRef<FConcertSessionActivity>> GetSelectedActivities() const;
	const TArray<TSharedPtr<FConcertSessionActivity>>& GetActivities() const { return ActivityListView->GetActivities(); }
	void SetSelectedActivities(const TArray<TSharedPtr<FConcertSessionActivity>>& ActivitiesToSelect) { ActivityListView->SetSelectedActivities(ActivitiesToSelect); }

	UE_API void OnColumnVisibilitySettingsChanged(const FColumnVisibilitySnapshot& ColumnSnapshot);

private:

	/** Invoked when the text in the search box widget changes. */
	UE_API void OnSearchTextChanged(const FText& InSearchText);

	/** Invoked when the text in the search box widget is committed. */
	UE_API void OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type CommitType);

	/** Returns the text to highlight when the search bar has a text set. */
	UE_API FText HighlightSearchedText() const;

private:
	
	/** Holds the map of endpoint IDs to client info. */
	TMap<FGuid, FConcertClientInfo> EndpointClientInfoMap;

	/** Holds the map of activity IDs to Concert activities. */
	TMap<int64, TSharedPtr<FConcertSessionActivity>> ActivityMap;

	/** Display the activity list. */
	TSharedPtr<SConcertSessionActivities> ActivityListView;

	/** Controls the activity list view options */
	TSharedPtr<FConcertSessionActivitiesOptions> ActivityListViewOptions;

	/** The widget used to enter the text to search. */
	TSharedPtr<SSearchBox> SearchBox;

	/** The searched text to highlight. */
	FText SearchedText;

	/** Used to limit activities. */
	FAllowActivity AllowActivityFunc;

	UE_API TOptional<FConcertClientInfo> GetClientInfo(FGuid Guid) const;
};

#undef UE_API
