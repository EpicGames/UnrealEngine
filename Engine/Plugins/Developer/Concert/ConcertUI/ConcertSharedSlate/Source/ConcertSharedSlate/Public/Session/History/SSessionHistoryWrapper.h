// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AbstractSessionHistoryController.h"
#include "SSessionHistory.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API CONCERTSHAREDSLATE_API

/** Shows a SSessionHistory widget - keeping alive the controller as long as this widget lives. Useful for closeable tabs. */
class SSessionHistoryWrapper : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSessionHistoryWrapper) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TSharedRef<FAbstractSessionHistoryController> InController);

private:

	/** Manages SessionHistory */
	TSharedPtr<FAbstractSessionHistoryController> Controller;
};

#undef UE_API
