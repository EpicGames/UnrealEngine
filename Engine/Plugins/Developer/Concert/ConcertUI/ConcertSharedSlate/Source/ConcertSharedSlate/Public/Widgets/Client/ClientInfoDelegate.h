// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessageData.h"
#include "Delegates/Delegate.h"

struct FGuid;
template<typename OptionalType> struct TOptional;

namespace UE::ConcertSharedSlate
{
	DECLARE_DELEGATE_RetVal_OneParam(bool, FIsLocalClient, const FGuid& ClientId);
	DECLARE_DELEGATE_RetVal_OneParam(TOptional<FConcertClientInfo>, FGetOptionalClientInfo, const FGuid& ClientEndpointId);
	/**
	 * Used by SHorizontalList to determine what should go in the parentheses behind the display name, e.g. "Display Name (You)".
	 * Return FText::GetEmpty() if nothing should go into the parentheses.
	 */
	DECLARE_DELEGATE_RetVal_OneParam(FText, FGetClientParenthesesContent, const FGuid& ClientId);

	FORCEINLINE FText EvaluateGetClientParenthesesContent(const FGetClientParenthesesContent& Getter, const FGuid& ClientId)
	{
		return Getter.IsBound() ? Getter.Execute(ClientId) : FText::GetEmpty();
	}
}