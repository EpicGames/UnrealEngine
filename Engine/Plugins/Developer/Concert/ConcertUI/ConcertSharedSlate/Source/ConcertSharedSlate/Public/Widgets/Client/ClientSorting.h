// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClientInfoDelegate.h"
#include "SClientName.h"
#include "Styling/AppStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class IConcertClient;
class SScrollBox;
class SWidgetSwitcher;

struct FConcertSessionClientInfo;
struct FGuid;

namespace UE::ConcertSharedSlate
{
	DECLARE_DELEGATE_RetVal_TwoParams(bool,
		FClientSortPredicate,
		const FConcertSessionClientInfo& Left,
		const FConcertSessionClientInfo& Right
		);

	template<typename TDelegate>
	concept CClientInfoSortable = std::is_convertible_v<TDelegate, const FGetClientParenthesesContent&>;

	/**
	 * Predicate for sorting everything that has ParenthesesContentToPlaceFirst first, and then sorts everything alphabetically.
	 *
	 * This is supposed to be used with SHorizontalClientList.
	 * Example input list: ["AClient", "BClient", "ZClient(You)"]
	 * Example output list sorted by ParenthesesContentToPlaceFirst == "You": ZClient(You), AClient, BClient 
	 */
	inline bool SortSpecifiedParenthesesFirstThenThenAlphabetical(
		const FConcertSessionClientInfo& Left,
		const FConcertSessionClientInfo& Right,
		const FGetClientParenthesesContent& GetClientParenthesesContent,
		const FText& ParenthesesContentToPlaceFirst
		)
	{
		const bool bLeftHasParenthesesContent = GetClientParenthesesContent.IsBound()
			&& GetClientParenthesesContent.Execute(Left.ClientEndpointId).EqualTo(ParenthesesContentToPlaceFirst);
		const bool bRightHasParenthesesContent = GetClientParenthesesContent.IsBound()
			&& GetClientParenthesesContent.Execute(Right.ClientEndpointId).EqualTo(ParenthesesContentToPlaceFirst);
		return bLeftHasParenthesesContent
			|| (!bRightHasParenthesesContent && Left.ClientInfo.DisplayName < Right.ClientInfo.DisplayName);
	}

	/** Sorts anything that has "You" in the parentheses first. */
	inline bool SortLocalClientParenthesesFirstThenThenAlphabetical(
		const FConcertSessionClientInfo& Left,
		const FConcertSessionClientInfo& Right,
		const FGetClientParenthesesContent& GetClientParenthesesContent
		)
	{
		return SortSpecifiedParenthesesFirstThenThenAlphabetical(
			Left, Right, GetClientParenthesesContent, ParenthesesClientNameContent::LocalClient
			);
	}
}