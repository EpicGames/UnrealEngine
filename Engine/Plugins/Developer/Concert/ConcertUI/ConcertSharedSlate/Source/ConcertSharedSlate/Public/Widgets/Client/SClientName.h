// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessageData.h"
#include "Styling/AppStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API CONCERTSHAREDSLATE_API

class IConcertClient;

/** Contains SClientName's parentheses content definitions. */
namespace UE::ConcertSharedSlate::ParenthesesClientNameContent
{
	/** The client corresponds to the local user. "You" is appended to name, e.g. "ClientName(You)". */
	CONCERTSHAREDSLATE_API extern const FText LocalClient;
	/** The client corresponds to a client that is not connected to the session. */
	CONCERTSHAREDSLATE_API extern const FText OfflineClient;
}

namespace UE::ConcertSharedSlate
{
	/**
	 * Knows how to display FConcertClientInfo.
	 * 
	 * The widget looks like this: []DisplayName(ParenthesesContent)
	 *  - [] is a square displaying the avatar colour (optional)
	 *  - DisplayName is FConcertClientInfo::DisplayName
	 *  - ParenthesesContent is additional info you can supply, like "You" (optional). See ParenthesesClientNameContent. 
	 */
	class SClientName : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SClientName)
			: _DisplayAvatarColor(true)
			, _Font(FAppStyle::Get().GetFontStyle("BoldFont"))
		{}
			/** The client info to display. */
			SLATE_ATTRIBUTE(TOptional<FConcertClientInfo>, ClientInfo)
			
			/** Content to display behind the display name in parentheses. */
			SLATE_ATTRIBUTE(FText, ParenthesisContent)
			/** Whether to show a square image in front of the name. */
			SLATE_ATTRIBUTE(bool, DisplayAvatarColor)
			
			/** Used for highlighting in the text */
			SLATE_ATTRIBUTE(FText, HighlightText)
			/** The font to use for the name */
			SLATE_ARGUMENT(FSlateFontInfo, Font)
		SLATE_END_ARGS()

		UE_API void Construct(const FArguments& InArgs);

		/** @return The display that would be used given the settings. */
		static UE_API FText GetDisplayText(const FConcertClientInfo& Info, bool bDisplayAsLocalClient);
		/** @return The display that would be used given the settings. */
		static UE_API FText GetDisplayText(const FConcertClientInfo& Info, const FText& ParenthesesContent = FText::GetEmpty());

	private:

		/** The client info to display. */
		TAttribute<TOptional<FConcertClientInfo>> ClientInfoAttribute;
		/** Content to display behind the display name in parentheses. */
		TAttribute<FText> ParenthesisContentAttribute;

		/** Gets the display name. */
		UE_API FText GetClientDisplayName() const;
		/** Gets the avatar color */
		UE_API FSlateColor GetAvatarColor() const;
	};
}

#undef UE_API
