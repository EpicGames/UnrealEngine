// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClientInfoDelegate.h"
#include "ClientSorting.h"
#include "Styling/AppStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API CONCERTSHAREDSLATE_API

class IConcertClient;
class SScrollBox;
class SWidgetSwitcher;

struct FConcertSessionClientInfo;
struct FGuid;

namespace UE::ConcertSharedSlate
{
	/** Aligns client widgets from left to right. If there is not enough space, a horizontal scroll bar cuts of the list. */
	class SHorizontalClientList : public SCompoundWidget
	{
	public:

		/** @return The display string a SHorizontalClientList would display with the given state. Returns unset optional if EmptyListSlot would be shown. */
		static UE_API TOptional<FString> GetDisplayString(
			const TConstArrayView<FGuid>& Clients,
			const FGetOptionalClientInfo& GetClientInfoDelegate,
			const FClientSortPredicate& SortPredicate,
			const FGetClientParenthesesContent& GetClientParenthesesContent = {}
			);
		
		SLATE_BEGIN_ARGS(SHorizontalClientList)
			: _Font(FAppStyle::Get().GetFontStyle("NormalFont"))
		{}
			/** Gets the content to place in parentheses behind the given client. */
			SLATE_EVENT(ConcertSharedSlate::FGetClientParenthesesContent, GetClientParenthesesContent)
			
			/** Used to get client display info for remote clients. */
			SLATE_EVENT(ConcertSharedSlate::FGetOptionalClientInfo, GetClientInfo)
			
			/** Whether to show a square image in front of the name. */
			SLATE_ATTRIBUTE(bool, DisplayAvatarColor)
			
			/** Used for highlighting in the text */
			SLATE_ATTRIBUTE(FText, HighlightText)
			/** The font to use for the names */
			SLATE_ARGUMENT(FSlateFontInfo, Font)
			
			/** Defaults to placing the local client first (if contained) and sorting alphabetically otherwise. */
			SLATE_EVENT(FClientSortPredicate, SortPredicate)

			/** Tooltip text to display when the list is non-empty. */
			SLATE_ATTRIBUTE(FText, ListToolTipText)
			
			/** The widget to display when the list is empty */
			SLATE_NAMED_SLOT(FArguments, EmptyListSlot)
		SLATE_END_ARGS()

		UE_API void Construct(const FArguments& InArgs);

		/** Refreshes the list. */
		UE_API void RefreshList(const TConstArrayView<FGuid>& Clients) const;

	private:
		
		/** Gets the content to place in parentheses behind the given client. */
		FGetClientParenthesesContent GetClientParenthesesContentDelegate;
		/** Used to get client display info for remote clients. */
		FGetOptionalClientInfo GetClientInfoDelegate;
		/** Sorts the client list */
		FClientSortPredicate SortPredicateDelegate;

		/** Whether the square in front of the client name should be displayed. */
		TAttribute<bool> ShouldDisplayAvatarColorAttribute;
		/** Used for highlighting in the text */
		TAttribute<FText> HighlightTextAttribute;
		
		/** The font to use for the names */
		FSlateFontInfo NameFont;

		/** Displays the ScrollBox when there are clients and the EmptyListSlot otherwise. */
		TSharedPtr<SWidgetSwitcher> WidgetSwitcher;
		/** Contains the children. */
		TSharedPtr<SScrollBox> ScrollBox;
	};
}

#undef UE_API
