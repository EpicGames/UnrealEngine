// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessageData.h"
#include "Misc/Optional.h"
#include "Styling/AppStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API CONCERTSHAREDSLATE_API

class IConcertClient;

namespace UE::ConcertSharedSlate
{
	/**
	 * Displays the name of a client.
	 * 
	 * The name will look like "Client Name (me)".
	 * @see SRemoteClientName
	 */
	class SLocalClientName : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SLocalClientName)
			: _Font(FAppStyle::Get().GetFontStyle("BoldFont"))
		{}
			/** The client info to display. */
			SLATE_ATTRIBUTE(TOptional<FConcertClientInfo>, DisplayInfo)
			
			/** Whether to show a square image in front of the name. */
			SLATE_ATTRIBUTE(bool, DisplayAvatarColor)
			
			/** Used for highlighting in the text */
			SLATE_ATTRIBUTE(FText, HighlightText)
			/** The font to use for the name */
			SLATE_ARGUMENT(FSlateFontInfo, Font)
		SLATE_END_ARGS()

		UE_API void Construct(const FArguments& InArgs);
	};
}

#undef UE_API
