// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessageData.h"
#include "Misc/Optional.h"
#include "Styling/AppStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API CONCERTSHAREDSLATE_API

class IConcertClient;

namespace UE::ConcertSharedSlate
{
	/**
	 * Displays the name of a client.
	 * 
	 * The name will look like "Client Name".
	 * @see SLocalClientName
	 *
	 * If the client disconnects, the last known info is used.
	 * If the client info is unknown, the widget will display an empty FConcertClientInfo;
	 */
	class SRemoteClientName : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SRemoteClientName)
			: _Font(FAppStyle::Get().GetFontStyle("BoldFont"))
		{}
			/** The client info to display. */
			SLATE_ATTRIBUTE(TOptional<FConcertClientInfo>, DisplayInfo)
			
			/** Whether to show a square image in front of the name. */
			SLATE_ATTRIBUTE(bool, DisplayAvatarColor)
			
			/** Used for highlighting in the text */
			SLATE_ATTRIBUTE(FText, HighlightText)
			/** The font to use for the name */
			SLATE_ARGUMENT(FSlateFontInfo, Font)
		SLATE_END_ARGS()

		UE_API void Construct(const FArguments& InArgs);

	private:

		/** The endpoint ID of the client to display. */
		TAttribute<TOptional<FConcertClientInfo>> ClientDisplayInfo;

		/**
		 * Cached so that the info remains known when the client disconnects.
		 * Must be mutable because TAttribute::CreateSP requires GetClientInfo to be const.
		 */
		mutable TOptional<FConcertClientInfo> LastKnownClientInfo;

		UE_API TOptional<FConcertClientInfo> GetClientInfo() const;
	};
}

#undef UE_API
